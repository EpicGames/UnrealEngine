// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MfMedia_WinGDK : MfMedia
	{
		public MfMedia_WinGDK(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDefinitions.Add("MFMEDIA_SUPPORTED_PLATFORM=1");
			PrivateDefinitions.Add("MFMEDIA_PLATFORM_SUPPORTS_ONLY_AAC=1");
			PrivateDefinitions.Add("MFMEDIA_PLATFORM_SUPPORTS_ONLY_H264=1");
			PrivateDefinitions.Add("MFMEDIA_PLATFORM_SUPPORTS_ONLY_NV12=1");
			PrivateDefinitions.Add("MFMEDIA_PLATFORM_SUPPORTS_ASYNC_TRACK_READER=1");
		}
	}
}
