// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
	#include <d3dcommon.h>
	#include <d3d12.h>
	#include <mfapi.h>
	#include <mferror.h>
	#include <mfidl.h>
	#include <Mfreadwrite.h>
	#include <uuids.h>
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"
#include "Microsoft/COMPointer.h"
