// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MfMediaFactory_WinGDK : MfMediaFactory
	{
		public MfMediaFactory_WinGDK(ReadOnlyTargetRules Target) : base(Target)
		{
			DynamicallyLoadedModuleNames.Add("MfMedia");
		}

		protected override bool DoAllowHTTPSPlayback()
		{
			return true;
		}
	}
}
