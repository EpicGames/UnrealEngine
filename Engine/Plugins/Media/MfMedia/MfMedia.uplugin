{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Media Foundation Media Player",
	"Description" : "Implements a media player using the Microsoft Media Foundation framework. Requires Xbox One or Windows 7 and higher.",
	"Category" : "Media Players",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "https://forums.unrealengine.com/showthread.php?46879-Media-Framework-Documentation-for-4-5-Preview",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "MfMedia",
			"Type" : "RuntimeNoCommandlet",
			"LoadingPhase" : "PreLoadingScreen",
			"HasExplicitPlatforms" : true,
			"PlatformAllowList" : []
		},
		{
			"Name" : "MfMediaEditor",
			"Type" : "Editor",
			"LoadingPhase" : "PostEngineInit"
		},
		{
			"Name" : "MfMediaFactory",
			"Type" : "Editor",
			"LoadingPhase" : "PostEngineInit"
		},
		{
			"Name" : "MfMediaFactory",
			"Type" : "RuntimeNoCommandlet",
			"LoadingPhase" : "PostEngineInit",
			"PlatformAllowList" : [ "Win64" ]
		}
	]
}
