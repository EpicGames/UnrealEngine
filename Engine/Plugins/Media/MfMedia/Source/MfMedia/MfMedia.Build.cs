// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MfMedia : ModuleRules
	{
		public MfMedia(ReadOnlyTargetRules Target) : base(Target)
		{
			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"Media",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"MediaUtils",
					"RenderCore",
				});

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"Media",
				});

			if (Target.Type != TargetType.Server)
			{
				if (Target.Platform == UnrealTargetPlatform.Win64)
				{
					PublicDelayLoadDLLs.Add("mf.dll");
					PublicDelayLoadDLLs.Add("mfplat.dll");
					PublicDelayLoadDLLs.Add("mfreadwrite.dll");
					PublicDelayLoadDLLs.Add("mfuuid.dll");
					PublicDelayLoadDLLs.Add("propsys.dll");
					PublicDelayLoadDLLs.Add("shlwapi.dll");
				}
				else
				{
					PublicSystemLibraries.Add("mfplat.lib");
					PublicSystemLibraries.Add("mfreadwrite.lib");
					PublicSystemLibraries.Add("mfuuid.lib");
				}
			}
		}
	}
}
