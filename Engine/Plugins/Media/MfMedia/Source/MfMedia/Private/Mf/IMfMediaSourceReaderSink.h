// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MfMediaPrivate.h"

#if MFMEDIA_SUPPORTED_PLATFORM

#include "Misc/Timespan.h"

#if PLATFORM_MICROSOFT
	#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#endif


/**
 * Interface for objects that receive media source reader callbacks.
 *
 * @see FMfMediaSourceReaderCallback
 */
class IMfMediaSourceReaderSink
{
public:

	/**
	 * Receive the given source reader event.
	 *
	 * @param Event The event to receive.
	 * @note Implementors must be thread-safe as this method may be called from arbitrary worker threads.
	 * @see ReceiveSourceReaderSample, ReceiveSourceReaderFlush
	 */
	virtual void ReceiveSourceReaderEvent(MediaEventType Event) = 0;

	/**
	 * Receive a source reader flush notification.
	 *
	 * @note Implementors must be thread-safe as this method may be called from arbitrary worker threads.
	 * @see ReceiveSourceReaderEvent, ReceiveSourceReaderSample
	 */
	virtual void ReceiveSourceReaderFlush() = 0;

	/**
	 * Receive the given media sample.
	 *
	 * @param Sample The sample to receive.
	 * @param Status The sample's status code.
	 * @param StreamFlags The stream flags.
	 * @param StreamIndex The index of the stream that generated the sample.
	 * @param Timestamp The sample's time stamp.
	 * @note Implementors must be thread-safe as this method may be called from arbitrary worker threads.
	 * @see ReceiveSourceReaderEvent, ReceiveSourceReaderFlush
	 */
	virtual void ReceiveSourceReaderSample(IMFSample* Sample, HRESULT Status, DWORD StreamFlags, DWORD StreamIndex, FTimespan Time) = 0;
};


#if PLATFORM_MICROSOFT
	#include "Microsoft/HideMicrosoftPlatformTypes.h"
#endif

#endif //MFMEDIA_SUPPORTED_PLATFORM
