// Copyright Epic Games, Inc. All Rights Reserved.

#include "MfMediaByteStream.h"

#if MFMEDIA_SUPPORTED_PLATFORM

#include "CoreTypes.h"
#include "HAL/PlatformAtomics.h"
#include "Misc/ScopeLock.h"
#include "MfMediaReadState.h"
#include "Serialization/Archive.h"
#include "Microsoft/COMPointer.h"

#if PLATFORM_MICROSOFT
	#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#endif


/* FMfByteStream structors
 *****************************************************************************/

FMfMediaByteStream::FMfMediaByteStream(const TSharedRef<FArchive, ESPMode::ThreadSafe>& InArchive)
	: AsyncReadInProgress(false)
	, Archive(InArchive)
	, RefCount(0)
{ }


/* IMFAsyncCallback interface
 *****************************************************************************/

STDMETHODIMP_(ULONG) FMfMediaByteStream::AddRef()
{
	return FPlatformAtomics::InterlockedIncrement(&RefCount);
}


STDMETHODIMP FMfMediaByteStream::GetParameters(unsigned long*, unsigned long*)
{
	return E_NOTIMPL;
}


STDMETHODIMP FMfMediaByteStream::Invoke(IMFAsyncResult* AsyncResult)
{
	// recover read state
	IUnknown* State = NULL;

	if (FAILED(AsyncResult->GetState(&State)))
	{
		return S_OK;
	}

	IMFAsyncResult* CallerResult = NULL;

	if (FAILED(State->QueryInterface(IID_PPV_ARGS(&CallerResult))))
	{
		return S_OK;
	}

	IUnknown* Unknown = NULL;

	if ((CallerResult != NULL) && (FAILED(CallerResult->GetObject(&Unknown))))
	{
		return S_OK;
	}

	FMfMediaReadState* ReadState = static_cast<FMfMediaReadState*>(Unknown);

	// perform the read
	ULONG cbRead;
	Read(ReadState->GetReadBuffer(), ReadState->GetReadBufferSize() - ReadState->GetBytesRead(), &cbRead);
	ReadState->AddBytesRead(cbRead);

	// notify caller
	if (CallerResult != NULL)
	{
		CallerResult->SetStatus(S_OK);
		::MFInvokeCallback(CallerResult);
	}

	// clean up
	if (CallerResult != NULL)
	{
		CallerResult->Release();
	}

	if (Unknown != NULL)
	{
		Unknown->Release();
	}

	if (State != NULL)
	{
		State->Release();
	}

	return S_OK;
}


STDMETHODIMP FMfMediaByteStream::QueryInterface(REFIID RefID, void** Object)
{
	if (Object == NULL)
	{
		return E_INVALIDARG;
	}

	if ((RefID == IID_IUnknown) || (RefID == IID_IMFAsyncCallback) || (RefID == IID_IMFByteStream))
	{
		*Object = (LPVOID)this;
		AddRef();

		return NOERROR;
	}

	*Object = NULL;

	return E_NOINTERFACE;
}


STDMETHODIMP_(ULONG) FMfMediaByteStream::Release()
{
	int32 CurrentRefCount = FPlatformAtomics::InterlockedDecrement(&RefCount);
	
	if (CurrentRefCount == 0)
	{
		delete this;
	}

	return CurrentRefCount;
}


/* IMFByteStream interface
 *****************************************************************************/

STDMETHODIMP FMfMediaByteStream::BeginRead(BYTE* pb, ULONG cb, IMFAsyncCallback* pCallback, IUnknown* punkState)
{
	if ((pCallback == NULL) || (pb == NULL))
	{
		return E_INVALIDARG;
	}

	TComPtr<FMfMediaReadState> ReadState = new(std::nothrow) FMfMediaReadState(pb, cb);

	if (ReadState == NULL)
	{
		return E_OUTOFMEMORY;
	}

	IMFAsyncResult* AsyncResult = NULL;
	HRESULT Result = ::MFCreateAsyncResult(ReadState, pCallback, punkState, &AsyncResult);

	if (SUCCEEDED(Result))
	{
		AsyncReadInProgress = true;
#if PLATFORM_WINDOWS
		Result = ::MFPutWorkItem(MFASYNC_CALLBACK_QUEUE_STANDARD, this, AsyncResult);
#else
		Result = ::MFPutWorkItem2(MFASYNC_CALLBACK_QUEUE_STANDARD, 0, this, AsyncResult);
#endif
		AsyncResult->Release();
	}

	return Result;
}


STDMETHODIMP FMfMediaByteStream::BeginWrite(const BYTE* pb, ULONG cb, IMFAsyncCallback* pCallback, IUnknown* punkState)
{
	return E_NOTIMPL;
}


STDMETHODIMP FMfMediaByteStream::Close()
{
	return S_OK;
}


STDMETHODIMP FMfMediaByteStream::EndRead(IMFAsyncResult* pResult, ULONG* pcbRead)
{
	if (pcbRead == NULL)
	{
		return E_INVALIDARG;
	}

	IUnknown* Unknown;
	{
		pResult->GetObject(&Unknown);
		FMfMediaReadState* ReadState = static_cast<FMfMediaReadState*>(Unknown);
		*pcbRead = (ULONG)ReadState->GetBytesRead();

		Unknown->Release();
	}

	AsyncReadInProgress = false;

	return S_OK;
}


STDMETHODIMP FMfMediaByteStream::EndWrite(IMFAsyncResult* pResult, ULONG* pcbWritten)
{
	return E_NOTIMPL;
}


STDMETHODIMP FMfMediaByteStream::Flush()
{
	return E_NOTIMPL;
}


STDMETHODIMP FMfMediaByteStream::GetCapabilities(DWORD* pdwCapabilities)
{
	FScopeLock ScopeLock(&CriticalSection);
	*pdwCapabilities = MFBYTESTREAM_IS_READABLE | MFBYTESTREAM_IS_SEEKABLE;

	return S_OK;
}


STDMETHODIMP FMfMediaByteStream::GetCurrentPosition(QWORD* pqwPosition)
{
	FScopeLock ScopeLock(&CriticalSection);
	*pqwPosition = Archive->Tell();

	return S_OK;
}


STDMETHODIMP FMfMediaByteStream::GetLength(QWORD* pqwLength)
{
	FScopeLock ScopeLock(&CriticalSection);
	*pqwLength = (QWORD)Archive->TotalSize();

	return S_OK;
}


STDMETHODIMP FMfMediaByteStream::IsEndOfStream(BOOL* pfEndOfStream)
{
	if (pfEndOfStream == NULL)
	{
		return E_INVALIDARG;
	}

	FScopeLock ScopeLock(&CriticalSection);
	*pfEndOfStream = Archive->AtEnd() ? TRUE : FALSE;

	return S_OK;
}


STDMETHODIMP FMfMediaByteStream::Read(BYTE* pb, ULONG cb, ULONG* pcbRead)
{
	FScopeLock ScopeLock(&CriticalSection);

	int64 Position = Archive->Tell();
	int64 Size = Archive->TotalSize();
	ULONG BytesToRead = cb;

	if (BytesToRead > (ULONG)Size)
	{
		BytesToRead = Size;
	}

	if ((Size - BytesToRead) < Archive->Tell())
	{
		BytesToRead = Size - Position;
	}

	if (BytesToRead > 0)
	{
		Archive->Serialize(pb, BytesToRead);
	}

	if (pcbRead != NULL)
	{
		*pcbRead = BytesToRead;
	}	

	Archive->Seek(Position + BytesToRead);

	return S_OK;
}


STDMETHODIMP FMfMediaByteStream::Seek(MFBYTESTREAM_SEEK_ORIGIN SeekOrigin, LONGLONG qwSeekOffset, DWORD dwSeekFlags, QWORD* pqwCurrentPosition)
{
	FScopeLock ScopeLock(&CriticalSection);

	if (AsyncReadInProgress)
	{
		return S_FALSE;
	}

	if (SeekOrigin == msoCurrent)
	{
		Archive->Seek(Archive->Tell() + qwSeekOffset);
	}
	else
	{
		Archive->Seek(qwSeekOffset);
	}

	if (pqwCurrentPosition != NULL)
	{
		*pqwCurrentPosition = (QWORD)Archive->Tell();
	}

	return S_OK;
}


STDMETHODIMP FMfMediaByteStream::SetLength(QWORD qwLength)
{
	return E_NOTIMPL;
}


STDMETHODIMP FMfMediaByteStream::SetCurrentPosition(QWORD qwPosition)
{
	FScopeLock ScopeLock(&CriticalSection);

	if (AsyncReadInProgress)
	{
		return S_FALSE;
	}

	if (qwPosition > (QWORD)Archive->TotalSize())
	{
		// MSDN is wrong: clamp instead of returning E_INVALIDARG
		qwPosition = (QWORD)Archive->TotalSize();
	}

	Archive->Seek((int64)qwPosition);

	return S_OK;
}


STDMETHODIMP FMfMediaByteStream::Write(const BYTE* pb, ULONG cb, ULONG* pcbWritten)
{
	return E_NOTIMPL;
}


#if PLATFORM_MICROSOFT
	#include "Microsoft/HideMicrosoftPlatformTypes.h"
#endif

#endif //MFMEDIA_SUPPORTED_PLATFORM
