// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MfMediaPrivate.h"

#if MFMEDIA_SUPPORTED_PLATFORM

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"

#if PLATFORM_MICROSOFT
	#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#endif

class FArchive;


/**
 * Implements a wrapper for media source data that is played from memory.
 */
class FMfMediaByteStream
	: public IMFAsyncCallback
	, public IMFByteStream
{
public:

	/**
	 * Creates and initializes a new instance from the specified buffer.
	 *
	 * @param InArchive The archive to stream from.
	 */
	FMfMediaByteStream(const TSharedRef<FArchive, ESPMode::ThreadSafe>& InArchive);

public:

	//~ IMFAsyncCallback interface

	STDMETHODIMP_(ULONG) AddRef();
	STDMETHODIMP GetParameters(unsigned long*, unsigned long*);
	STDMETHODIMP Invoke(IMFAsyncResult* AsyncResult);
	STDMETHODIMP QueryInterface(REFIID RefID, void** Object);
	STDMETHODIMP_(ULONG) Release();

public:

	//~ IMFByteStream interface

	STDMETHODIMP BeginRead(BYTE* pb, ULONG cb, IMFAsyncCallback* pCallback, IUnknown* punkState);
	STDMETHODIMP BeginWrite(const BYTE* pb, ULONG cb, IMFAsyncCallback* pCallback, IUnknown* punkState);
	STDMETHODIMP Close();
	STDMETHODIMP EndRead(IMFAsyncResult* pResult, ULONG* pcbRead);
	STDMETHODIMP EndWrite(IMFAsyncResult* pResult, ULONG* pcbWritten);
	STDMETHODIMP Flush();
	STDMETHODIMP GetCapabilities(DWORD* pdwCapabilities);
	STDMETHODIMP GetCurrentPosition(QWORD* pqwPosition);
	STDMETHODIMP GetLength(QWORD* pqwLength);
	STDMETHODIMP IsEndOfStream(BOOL* pfEndOfStream);
	STDMETHODIMP Read(BYTE* pb, ULONG cb, ULONG* pcbRead);
	STDMETHODIMP Seek(MFBYTESTREAM_SEEK_ORIGIN SeekOrigin, LONGLONG qwSeekOffset, DWORD dwSeekFlags, QWORD* pqwCurrentPosition);
	STDMETHODIMP SetLength(QWORD qwLength);
	STDMETHODIMP SetCurrentPosition(QWORD qwPosition);
	STDMETHODIMP Write(const BYTE* pb, ULONG cb, ULONG* pcbWritten);

protected:

	/** Hidden destructor. */
	virtual ~FMfMediaByteStream() = default;

protected:

	/**
	 * Open media source from a byte stream.
	 *
	 * @param Archive The archive holding the media data.
	 * @param OriginalUrl The original URL of the media that was loaded into the buffer.
	 * @return true if the media was opened, false otherwise.
	 * @see OpenUrl
	 */
	bool OpenArchive(const TSharedRef<FArchive, ESPMode::ThreadSafe>& Archive, const FString& OriginalUrl);

	/**
	 * Open media source from a file or internet URL.
	 *
	 * @param Url The URL of the media to open (file name or web address).
	 * @return true if the media was opened, false otherwise.
	 * @see OpenArchive
	 */
	bool OpenUrl(const FString& Url);

private:

	/** Whether the stream is currently being read asynchronously. */
	bool AsyncReadInProgress;

	/** Holds the archive to stream from. */
	TSharedRef<FArchive, ESPMode::ThreadSafe> Archive;

	/** Critical section for locking access to this class. */
	mutable FCriticalSection CriticalSection;

	/** Holds a reference counter for this instance. */
	int32 RefCount;
};


#if PLATFORM_MICROSOFT
#include "Microsoft/HideMicrosoftPlatformTypes.h"
#endif

#endif //MFMEDIA_SUPPORTED_PLATFORM
