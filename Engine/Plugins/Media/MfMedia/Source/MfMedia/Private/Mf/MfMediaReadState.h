// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MfMediaPrivate.h"

#if MFMEDIA_SUPPORTED_PLATFORM

#if PLATFORM_MICROSOFT
	#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#endif


/**
 * Implements the state information for asynchronous reads of byte buffer sources.
 *
 * The @see FMfByteStream class uses this class to store the state for asynchronous
 * read requests that are initiated with @see FMfByteStream.BeginRead and completed
 * with @see FMfByteStream.EndRead
 */
class FMfMediaReadState
	: public IUnknown
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InReadBuffer The buffer that receives the read data.
	 * @param InReadBufferSize The size of the read buffer.
	 */
	FMfMediaReadState(BYTE* InReadBuffer, ULONG InReadBufferSize)
		: BytesRead(0)
		, ReadBuffer(InReadBuffer)
		, ReadBufferSize(InReadBufferSize)
		, RefCount(0)
	{ }

public:

	/**
	 * Adds the specified number of bytes read.
	 *
	 * @param BytesToAdd The number of bytes to add.
	 */
	void AddBytesRead(uint64 BytesToAdd)
	{
		BytesRead += BytesToAdd;
	}

	/**
	 * Gets a pointer to the buffer being read from.
	 *
	 * @return The buffer.
	 */
	BYTE* GetReadBuffer() const
	{
		return ReadBuffer;
	}

	/**
	 * Gets the size of the buffer being read from.
	 *
	 * @return The size of the buffer (in bytes).
	 */
	ULONG GetReadBufferSize() const
	{
		return ReadBufferSize;
	}

	/**
	 * Gets the actual number of bytes read.
	 *
	 * @return Bytes read count.
	 */
	uint64 GetBytesRead() const
	{
		return BytesRead;
	}

public:

	//~ IUnknown interface

	STDMETHODIMP QueryInterface(REFIID RefID, void** Object)
	{
		if (Object == NULL)
		{
			return E_INVALIDARG;
		}

		if (RefID == IID_IUnknown)
		{
			*Object = (LPVOID)this;
			AddRef();

			return NOERROR;
		}

		*Object = NULL;

		return E_NOINTERFACE;
	}

	STDMETHODIMP_(ULONG) AddRef()
	{
		return FPlatformAtomics::InterlockedIncrement(&RefCount);
	}

	STDMETHODIMP_(ULONG) Release()
	{
		int32 CurrentRefCount = FPlatformAtomics::InterlockedDecrement(&RefCount);
	
		if (CurrentRefCount == 0)
		{
			delete this;
		}

		return CurrentRefCount;
	}

protected:

	/** Hidden destructor. */
	virtual ~FMfMediaReadState() = default;

private:

	/** Number of bytes read. */
	uint64 BytesRead;

	/** The buffer that receives the read data. */
	BYTE* ReadBuffer;

	/** The size of the read buffer. */
	ULONG ReadBufferSize;

	/** Holds a reference counter for this instance. */
	int32 RefCount;
};


#if PLATFORM_MICROSOFT
	#include "Microsoft/HideMicrosoftPlatformTypes.h"
#endif

#endif //MFMEDIA_SUPPORTED_PLATFORM
