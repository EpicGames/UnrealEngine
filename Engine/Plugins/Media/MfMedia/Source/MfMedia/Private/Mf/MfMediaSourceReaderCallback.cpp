// Copyright Epic Games, Inc. All Rights Reserved.

#include "MfMediaSourceReaderCallback.h"
#include "MfMediaPrivate.h"

#if MFMEDIA_SUPPORTED_PLATFORM

#include "HAL/PlatformAtomics.h"

#include "IMfMediaSourceReaderSink.h"
#include "MfMediaUtils.h"

#if PLATFORM_MICROSOFT
	#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#endif

#define MFMEDIASOURCEREADERCALLBACK_TRACE_SAMPLES 0


/* FMfMediaSourceReaderCallback structors
 *****************************************************************************/

FMfMediaSourceReaderCallback::FMfMediaSourceReaderCallback(IMfMediaSourceReaderSink& InSink)
	: RefCount(0)
	, Sink(InSink)
{ }


FMfMediaSourceReaderCallback::~FMfMediaSourceReaderCallback()
{
	UE_LOGF(LogMfMedia, VeryVerbose, "Callback %p: Destroyed)", this);
}


/* IMFSourceReaderCallback interface
 *****************************************************************************/

STDMETHODIMP FMfMediaSourceReaderCallback::OnEvent(DWORD, IMFMediaEvent* Event)
{
	// get event type
	MediaEventType EventType = MEUnknown;
	{
		const HRESULT Result = Event->GetType(&EventType);

		if (FAILED(Result))
		{
			UE_LOGF(LogMfMedia, VeryVerbose, "Callback %p: Failed to get session event type: %ls", this, *MfMedia::ResultToString(Result));
			return Result;
		}
	}

	// get event status
	HRESULT EventStatus = S_FALSE;
	{
		const HRESULT Result = Event->GetStatus(&EventStatus);

		if (FAILED(Result))
		{
			UE_LOGF(LogMfMedia, VeryVerbose, "Callback %p: Failed to get event status: %ls", this, *MfMedia::ResultToString(Result));
			return Result;
		}
	}

	UE_LOGF(LogMfMedia, VeryVerbose, "Callback %p: Event [%ls]: %ls", this, *MfMedia::MediaEventToString(EventType), *MfMedia::ResultToString(EventStatus));

	Sink.ReceiveSourceReaderEvent(EventType);

	return S_OK;
}


STDMETHODIMP FMfMediaSourceReaderCallback::OnFlush(DWORD)
{
	Sink.ReceiveSourceReaderFlush();

	return S_OK;
}


STDMETHODIMP FMfMediaSourceReaderCallback::OnReadSample(HRESULT Status, DWORD StreamIndex, DWORD StreamFlags, LONGLONG Timestamp, IMFSample* Sample)
{
	#if MFMEDIASOURCEREADERCALLBACK_TRACE_SAMPLES
		UE_LOGF(LogMfMedia, VeryVerbose, "Callback %p: Sample read: %ls (stream = %i, status = %i, flags = %i)", this, *FTimespan(Timestamp).ToString(), StreamIndex, Status, StreamFlags);
	#endif

	Sink.ReceiveSourceReaderSample(Sample, Status, StreamFlags, StreamIndex, FTimespan(Timestamp));

	return S_OK;
}


/* IUnknown interface
 *****************************************************************************/

STDMETHODIMP_(ULONG) FMfMediaSourceReaderCallback::AddRef()
{
	return FPlatformAtomics::InterlockedIncrement(&RefCount);
}


#if _MSC_VER == 1900
	#pragma warning(push)
	#pragma warning(disable:4838)
#endif

STDMETHODIMP FMfMediaSourceReaderCallback::QueryInterface(REFIID RefID, void** Object)
{
	if (Object == NULL)
	{
		return E_INVALIDARG;
	}

	if ((RefID == IID_IUnknown) || (RefID == IID_IMFSourceReaderCallback))
	{
		*Object = (LPVOID)this;
		AddRef();

		return NOERROR;
	}

	*Object = NULL;

	return E_NOINTERFACE;
}

#if _MSC_VER == 1900
	#pragma warning(pop)
#endif


STDMETHODIMP_(ULONG) FMfMediaSourceReaderCallback::Release()
{
	int32 CurrentRefCount = FPlatformAtomics::InterlockedDecrement(&RefCount);

	if (CurrentRefCount == 0)
	{
		delete this;
	}

	return CurrentRefCount;
}


#if PLATFORM_MICROSOFT
	#include "Microsoft/HideMicrosoftPlatformTypes.h"
#endif

#endif //MFMEDIA_SUPPORTED_PLATFORM
