// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MfMediaPrivate.h"

#if MFMEDIA_SUPPORTED_PLATFORM

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"

#if PLATFORM_MICROSOFT
	#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#endif

class IMfMediaSourceReaderSink;


/**
 * Callback handler for Media Framework source readers.
 */
class FMfMediaSourceReaderCallback
	: public IMFSourceReaderCallback
{
public:

	/**
	 * Create and initialize a new instance.
	 *
	 * @param InSink The object that receives the callbacks.
	 */
	FMfMediaSourceReaderCallback(IMfMediaSourceReaderSink& InSink);

public:

	//~ IMFSourceReaderCallback interface

	STDMETHODIMP OnEvent(DWORD, IMFMediaEvent* Event);
	STDMETHODIMP OnFlush(DWORD);
	STDMETHODIMP OnReadSample(HRESULT Status, DWORD StreamIndex, DWORD StreamFlags, LONGLONG Timestamp, IMFSample* Sample);

public:

	//~ IUnknown interface

	STDMETHODIMP_(ULONG) AddRef();
	STDMETHODIMP QueryInterface(REFIID RefID, void** Object);
	STDMETHODIMP_(ULONG) Release();

protected:

	/** Hidden destructor (life time is managed by COM). */
	virtual ~FMfMediaSourceReaderCallback();

private:

	/** Holds a reference counter for this instance. */
	int32 RefCount;

	/** The object that receives the callbacks. */
	IMfMediaSourceReaderSink& Sink;
};


#if PLATFORM_MICROSOFT
	#include "Microsoft/HideMicrosoftPlatformTypes.h"
#endif

#endif //MFMEDIA_SUPPORTED_PLATFORM
