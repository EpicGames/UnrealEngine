// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MfMediaPrivate.h"

#if MFMEDIA_SUPPORTED_PLATFORM

#include "Containers/UnrealString.h"
#include "Templates/SharedPointer.h"

#if PLATFORM_MICROSOFT
	#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#endif

class FArchive;


namespace MfMedia
{
	/**
	 * Create an output media type for the given input media type.
	 *
	 * @param MajorType The output's major type, i.e. audio or video.
	 * @param SubType The output's sub type, i.e. PCM or RGB32.
	 * @param AllowNonStandardCodecs Whether non-standard codecs should be allowed.
	 * @return The output type, or NULL on error.
	 */
	TComPtr<IMFMediaType> CreateOutputType(const GUID& MajorType, const GUID& SubType, bool AllowNonStandardCodecs);

	/** 
	 * Convert a FOURCC code to string.
	 *
	 * @param Fourcc The code to convert.
	 * @return The corresponding string.
	 */
	FString FourccToString(unsigned long Fourcc);

	/**
	 * Convert a Windows GUID to string.
	 *
	 * @param Guid The GUID to convert.
	 * @return The corresponding string.
	 */
	FString GuidToString(const GUID& Guid);

	/**
	 * Convert a media major type to string.
	 *
	 * @param MajorType The major type GUID to convert.
	 * @return The corresponding string.
	 */
	FString MajorTypeToString(const GUID& MajorType);

	/**
	 * Convert a media event to string.
	 *
	 * @param Event The event code to convert.
	 * @return The corresponding string.
	 */
	FString MediaEventToString(MediaEventType Event);

	/**
	 * Resolve a media source from an archive or URL.
	 *
	 * @param Archive The archive to read media data from (optional).
	 * @param Url The media source URL.
	 * @param Precache Whether to precache media into RAM if URL is a local file.
	 * @return The media source object, or NULL if it couldn't be resolved.
	 */
	TComPtr<IMFMediaSource> ResolveMediaSource(TSharedPtr<FArchive, ESPMode::ThreadSafe> Archive, const FString& Url, bool Precache);

	/**
	 * Convert an WMF HRESULT code to string.
	 *
	 * @param Result The result code to convert.
	 * @return The corresponding string.
	 */
	FString ResultToString(HRESULT Result);

	/**
	 * Convert a media sub-type to string.
	 *
	 * @param SubType The sub-type GUID to convert.
	 * @return The corresponding string.
	 */
	FString SubTypeToString(const GUID& SubType);
}


#if PLATFORM_MICROSOFT
	#include "Microsoft/HideMicrosoftPlatformTypes.h"
#endif

#endif //MFMEDIA_SUPPORTED_PLATFORM
