// Copyright Epic Games, Inc. All Rights Reserved.

#include "MfMediaPrivate.h"

#include "Modules/ModuleManager.h"

#if MFMEDIA_SUPPORTED_PLATFORM
	#include "Player/MfMediaPlayer.h"
	#include "Templates/SharedPointer.h"
#endif

#include "IMfMediaModule.h"

#if PLATFORM_WINDOWS
	#pragma comment(lib, "mf")
	#pragma comment(lib, "mfplat")
	#pragma comment(lib, "mfreadwrite")
	#pragma comment(lib, "mfuuid")
	#pragma comment(lib, "propsys")
	#pragma comment(lib, "shlwapi")
#endif


DEFINE_LOG_CATEGORY(LogMfMedia);


/**
 * Implements the MfMedia module.
 */
class FMfMediaModule
	: public IMfMediaModule
{
public:

	/** Default constructor. */
	FMfMediaModule()
		: Initialized(false)
	{ }

public:

	//~ IMfMediaModule interface

	virtual TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CreatePlayer(IMediaEventSink& EventSink) override
	{
#if MFMEDIA_SUPPORTED_PLATFORM
		if (Initialized)
		{
			return MakeShared<FMfMediaPlayer, ESPMode::ThreadSafe>(EventSink);
		}
#endif
		return nullptr;
	}

public:

	//~ IModuleInterface interface

	virtual void StartupModule() override
	{
#if MFMEDIA_SUPPORTED_PLATFORM
		// initialize Windows Media Foundation
		HRESULT Result = MFStartup(MF_VERSION);

		if (FAILED(Result))
		{
			UE_LOGF(LogMfMedia, Log, "Failed to initialize Xbox One Media Foundation, Error %i", Result);

			return;
		}

		Initialized = true;
#endif
	}

	virtual void ShutdownModule() override
	{
#if MFMEDIA_SUPPORTED_PLATFORM
		Initialized = false;

		if (Initialized)
		{
			// shutdown Windows Media Foundation
			MFShutdown();
		}
#endif
	}

private:

	/** Whether the module has been initialized. */
	bool Initialized;
};


IMPLEMENT_MODULE(FMfMediaModule, MfMedia);
