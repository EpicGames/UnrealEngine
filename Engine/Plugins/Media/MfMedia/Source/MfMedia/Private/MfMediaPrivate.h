// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"

#if !defined(MFMEDIA_SUPPORTED_PLATFORM)
	#define MFMEDIA_SUPPORTED_PLATFORM (PLATFORM_WINDOWS && !UE_SERVER)
	#define MFMEDIA_NEED_PLATFORM_PRIVATE 0

#elif !defined(MFMEDIA_NEED_PLATFORM_PRIVATE)
	#define MFMEDIA_NEED_PLATFORM_PRIVATE 1

#endif



#if MFMEDIA_SUPPORTED_PLATFORM && MFMEDIA_NEED_PLATFORM_PRIVATE
	#include "MfMediaPlatformPrivate.h"

#elif MFMEDIA_SUPPORTED_PLATFORM
	#include "Microsoft/AllowMicrosoftPlatformTypes.h"

	#if PLATFORM_WINDOWS
		#include <windows.h>
		#include <propvarutil.h>
		#include <shlwapi.h>

		const GUID FORMAT_VideoInfo = { 0x05589f80, 0xc356, 0x11ce, { 0xbf, 0x01, 0x00, 0xaa, 0x00, 0x55, 0x59, 0x5a } };
		const GUID FORMAT_VideoInfo2 = { 0xf72a76A0, 0xeb0a, 0x11d0, { 0xac, 0xe4, 0x00, 0x00, 0xc0, 0xcc, 0x16, 0xba } };

		#if (WINVER < _WIN32_WINNT_WIN8)
			const GUID MF_LOW_LATENCY = { 0x9c27891a, 0xed7a, 0x40e1, { 0x88, 0xe8, 0xb2, 0x27, 0x27, 0xa0, 0x24, 0xee } };
		#endif
	#endif

	#include <mfapi.h>
	#include <mferror.h>
	#include <mfidl.h>
	#include <Mfreadwrite.h>

	#include "Microsoft/COMPointer.h"
	#include "Microsoft/HideMicrosoftPlatformTypes.h"

#elif PLATFORM_WINDOWS && !UE_SERVER
	#pragma message("Skipping MfMedia (requires WINVER >= 0x0601, but WINVER is " UE_STRINGIZE(WINVER) ")")

#endif //MFMEDIA_SUPPORTED_PLATFORM


/** Log category for the MfMedia module. */
DECLARE_LOG_CATEGORY_EXTERN(LogMfMedia, Log, All);
