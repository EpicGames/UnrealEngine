// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MfMediaPrivate.h"

#if MFMEDIA_SUPPORTED_PLATFORM

#include "CoreTypes.h"
#include "IMediaAudioSample.h"
#include "MediaObjectPool.h"
#include "MediaSampleQueue.h"
#include "Misc/Timespan.h"
#include "Microsoft/COMPointer.h"

#include "MfMediaSample.h"
#include "Mf/MfMediaUtils.h"


/**
 * Audio sample generated by MfMedia player.
 */
class FMfMediaAudioSample
	: public IMediaAudioSample
	, public IMediaPoolable
	, protected FMfMediaSample
{
public:

	/** Default constructor. */
	FMfMediaAudioSample()
		: BytesPerSample(0)
		, NumChannels(0)
		, SampleRate(0)
	{ }

	/** Virtual destructor. */
	virtual ~FMfMediaAudioSample() = default;

public:

	/**
	 * Initialize the sample.
	 *
	 * @param InMediaType The sample's media type.
	 * @param InSample The WMF sample.
	 */
	bool Initialize(IMFMediaType& InMediaType, IMFSample& InSample, uint32 InNumChannels, uint32 InSampleRate)
	{
		if (!InitializeSample(InSample))
		{
			return false;
		}

		// get audio sample specs
		BytesPerSample = ::MFGetAttributeUINT32(&InMediaType, MF_MT_AUDIO_BITS_PER_SAMPLE, 0) >> 3;
		NumChannels = InNumChannels; //::MFGetAttributeUINT32(&InMediaType, MF_MT_AUDIO_NUM_CHANNELS, 0);
		SampleRate = InSampleRate; //::MFGetAttributeUINT32(&InMediaType, MF_MT_AUDIO_SAMPLES_PER_SECOND, 0);

		if ((BytesPerSample == 0) || (NumChannels == 0) || (SampleRate == 0))
		{
			return false;
		}

		// get sample buffer
		TComPtr<IMFMediaBuffer> Buffer = NULL;
		{
			HRESULT Result = InSample.ConvertToContiguousBuffer(&Buffer);

			if (FAILED(Result))
			{
				UE_LOGF(LogMfMedia, VeryVerbose, "Failed to get contiguous buffer from audio sample (%ls)", *MfMedia::ResultToString(Result));
				return false;
			}
		}

		BYTE* Bytes = NULL;
		DWORD BufferSize = 0;
		{
			HRESULT Result = Buffer->Lock(&Bytes, NULL, &BufferSize);

			if (FAILED(Result))
			{
				UE_LOGF(LogMfMedia, VeryVerbose, "Failed to lock contiguous buffer from audio sample (%ls)", *MfMedia::ResultToString(Result));
				return false;
			}
		}

		if ((Bytes != NULL) && (BufferSize > 0))
		{
			Data.Reset(BufferSize);
			Data.Append(Bytes, BufferSize);
		}

		Buffer->Unlock();

		// duration provided by MF is sometimes incorrect when seeking
		Duration = (BufferSize * ETimespan::TicksPerSecond) / (NumChannels * SampleRate * BytesPerSample);

		return true;
	}

public:

	//~ IMediaAudioSample interface

	virtual const void* GetBuffer() override
	{
		return Data.GetData();
	}

	virtual uint32 GetChannels() const override
	{
		return NumChannels;
	}

	virtual FTimespan GetDuration() const override
	{
		return Duration;
	}

	virtual EMediaAudioSampleFormat GetFormat() const override
	{
		return EMediaAudioSampleFormat::Int16;
	}

	virtual uint32 GetFrames() const override
	{
		return Data.Num() / (NumChannels * BytesPerSample);
	}

	virtual uint32 GetSampleRate() const override
	{
		return SampleRate;
	}

	virtual FMediaTimeStamp GetTime() const override
	{
		return FMediaTimeStamp(Time);
	}

private:

	/** Number of bytes per audio sample. */
	uint32 BytesPerSample;

	/** The sample's data buffer. */
	TArray<uint8> Data;

	/** Number of audio channels. */
	uint32 NumChannels;

	/** Audio sample rate (in samples per second). */
	uint32 SampleRate;
};


/** Implements a pool for MF audio samples. */
class FMfMediaAudioSamplePool : public TMediaObjectPool<FMfMediaAudioSample> { };


#endif //MFMEDIA_SUPPORTED_PLATFORM
