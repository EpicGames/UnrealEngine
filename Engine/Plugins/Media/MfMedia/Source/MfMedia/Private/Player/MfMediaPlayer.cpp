// Copyright Epic Games, Inc. All Rights Reserved.

#include "MfMediaPlayer.h"

#if MFMEDIA_SUPPORTED_PLATFORM

#include "Async/Async.h"
#include "IMediaEventSink.h"
#include "IMediaOptions.h"
#include "MediaSamples.h"
#include "MediaPlayerOptions.h"

#include "Mf/MfMediaSourceReaderCallback.h"
#include "MfMediaTracks.h"
#include "Mf/MfMediaUtils.h"

#if PLATFORM_MICROSOFT
	#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#endif

#define LOCTEXT_NAMESPACE "FMfMediaPlayer"


/* FMfVideoPlayer structors
 *****************************************************************************/

FMfMediaPlayer::FMfMediaPlayer(IMediaEventSink& InEventSink)
	: Characteristics(0)
	, CurrentDuration(FTimespan::Zero())
	, CurrentRate(0.0f)
	, CurrentState(EMediaState::Closed)
	, CurrentStatus(EMediaStatus::None)
	, CurrentTime(FTimespan::Zero())
	, EventSink(InEventSink)
	, PlaybackRestarted(true)
	, Samples(MakeShared<FMediaSamples, ESPMode::ThreadSafe>())
	, ShouldLoop(false)
	, SourceReaderError(false)
	, Tracks(MakeShared<FMfMediaTracks, ESPMode::ThreadSafe>())
{ }


FMfMediaPlayer::~FMfMediaPlayer()
{
	Close();
}


/* IMediaPlayer interface
 *****************************************************************************/

void FMfMediaPlayer::Close()
{
	if (CurrentState == EMediaState::Closed)
	{
		return;
	}

	if (SourceReader)
	{
		SourceReader->Flush(MF_SOURCE_READER_ALL_STREAMS);
		SourceReader.Reset();
	}
	SourceReaderCallback.Reset();

	// reset player
	if (MediaSource != NULL)
	{
		MediaSource->Shutdown();
		MediaSource = NULL;
	}

	Tracks->Shutdown();

	PresentationDescriptor.Reset();
	RateControl.Reset();
	RateSupport.Reset();

	Characteristics = 0;
	CurrentDuration = FTimespan::Zero();
	CurrentRate = 0.0f;
	CurrentState = EMediaState::Closed;
	CurrentStatus = EMediaStatus::None;
	CurrentTime = FTimespan::Zero();
	MediaUrl = FString();
//	PendingSeekTime.Reset();
	PlaybackRestarted = true;
	SourceReaderError = false;
	ThinnedRates.Empty();
	UnthinnedRates.Empty();

	// notify listeners
	EventSink.ReceiveMediaEvent(EMediaEvent::TracksChanged);
	EventSink.ReceiveMediaEvent(EMediaEvent::MediaClosed);
}


IMediaCache& FMfMediaPlayer::GetCache()
{
	return *this;
}


IMediaControls& FMfMediaPlayer::GetControls()
{
	return *this;
}


FString FMfMediaPlayer::GetInfo() const
{
	return Tracks->GetInfo();
}


FGuid FMfMediaPlayer::GetPlayerPluginGUID() const
{
	static FGuid PlayerPluginGUID(0x6a5bd063, 0xe0854163, 0x867e5978, 0xf3eaa9f2);
	return PlayerPluginGUID;
}


IMediaSamples& FMfMediaPlayer::GetSamples()
{
	check(Samples.IsValid());
	return*Samples.Get();;
}


FString FMfMediaPlayer::GetStats() const
{
	FString Result;
	Tracks->AppendStats(Result);

	return Result;
}


IMediaTracks& FMfMediaPlayer::GetTracks()
{
	return *Tracks;
}


FString FMfMediaPlayer::GetUrl() const
{
	return MediaUrl;
}


IMediaView& FMfMediaPlayer::GetView()
{
	return *this;
}


bool FMfMediaPlayer::Open(const FString& Url, const IMediaOptions* Options, const FMediaPlayerOptions* PlayerOptions)
{
	Close();

	if (Url.IsEmpty())
	{
		return false;
	}

	const bool Precache = (Options != nullptr) ? Options->GetMediaOption("PrecacheFile", false) : false;

	return InitializePlayer(nullptr, Url, Precache, PlayerOptions);
}


bool FMfMediaPlayer::Open(const FString& Url, const IMediaOptions* Options)
{
	return Open(Url, Options, nullptr);
}


bool FMfMediaPlayer::Open(const TSharedRef<FArchive, ESPMode::ThreadSafe>& Archive, const FString& OriginalUrl, const IMediaOptions* /*Options*/)
{
	Close();

	if (Archive->TotalSize() == 0)
	{
		UE_LOGF(LogMfMedia, Verbose, "Player %p: Cannot open media from archive (archive is empty).", this);
		return false;
	}

	if (OriginalUrl.IsEmpty())
	{
		UE_LOGF(LogMfMedia, Verbose, "Player %p: Cannot open media from archive (no original URL provided).", this);
		return false;
	}

	return InitializePlayer(Archive, OriginalUrl, false, nullptr);
}


void FMfMediaPlayer::TickAudio()
{
	if (CurrentState == EMediaState::Playing)
	{
		Tracks->TickAudio(CurrentRate, CurrentTime);
	}
}


void FMfMediaPlayer::TickFetch(FTimespan /*DeltaTime*/, FTimespan /*Timecode*/)
{
	bool MediaSourceChanged = false;
	bool TrackSelectionChanged = false;

	Tracks->GetFlags(MediaSourceChanged, TrackSelectionChanged);

	if (MediaSourceChanged)
	{
		if (Tracks->IsInitialized())
		{
			EventSink.ReceiveMediaEvent(EMediaEvent::TracksChanged);

			if (CurrentState == EMediaState::Preparing)
			{
				CurrentDuration = Tracks->GetDuration();
				MediaSource = Tracks->GetMediaSource();
				SourceReader = Tracks->GetSourceReader();

				UpdateCharacteristics();

				if (MediaSource.IsValid())
				{
					CurrentState = EMediaState::Stopped;
					EventSink.ReceiveMediaEvent(EMediaEvent::MediaOpened);
				}
				else
				{
					CurrentState = EMediaState::Error;
					EventSink.ReceiveMediaEvent(EMediaEvent::MediaOpenFailed);
				}
			}
		}
		else
		{
			CurrentState = EMediaState::Error;
			EventSink.ReceiveMediaEvent(EMediaEvent::MediaOpenFailed);
		}
	}

	if (TrackSelectionChanged)
	{
		if (CommitTime(CurrentTime))
		{
			Tracks->Restart();
		}
	}

	if (MediaSourceChanged || TrackSelectionChanged)
	{
		Tracks->ClearFlags();
	}
}


void FMfMediaPlayer::TickInput(FTimespan DeltaTime, FTimespan Timecode)
{
	if (CurrentDuration == FTimespan::Zero())
	{
		return; // nothing to play
	}

	if ((CurrentState != EMediaState::Paused) && (CurrentState != EMediaState::Playing))
	{
		return; // not paused or playing
	}

	if (SourceReaderError)
	{
		CurrentState = EMediaState::Error;

		return;
	}

	// update clock
	if (PlaybackRestarted)
	{
		PlaybackRestarted = false;
	}
	else
	{
		CurrentTime += DeltaTime * CurrentRate;
	}

	// handle looping
	if ((CurrentTime >= CurrentDuration) || (CurrentTime < FTimespan::Zero()))
	{
		EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackEndReached);

		if (ShouldLoop)
		{
			if (CurrentRate > 0.0f)
			{
				CurrentTime = FTimespan::Zero();
			}
			else
			{
				CurrentTime = CurrentDuration - FTimespan(1);
			}

			PlaybackRestarted = true;
		}
		else
		{
			CurrentRate = 0.0f;
			CurrentState = EMediaState::Stopped;
			CurrentTime = FTimespan::Zero();

			EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackSuspended);
		}

		if (CommitTime(CurrentTime))
		{
			Tracks->Restart();
		}
	}

	// update tracks
	if ((CurrentState == EMediaState::Paused) || (CurrentState == EMediaState::Playing))
	{
		Tracks->TickInput(CurrentRate, CurrentTime);
	}
}


/* FMfMediaPlayer implementation
 *****************************************************************************/

bool FMfMediaPlayer::CommitTime(FTimespan Time)
{
	if (SourceReader == NULL)
	{
		return false;
	}

	// perform seek
	PROPVARIANT Position;

	if (Time >= FTimespan::Zero())
	{
		Position.vt = VT_I8;
		Position.hVal.QuadPart = Time.GetTicks();
	}
	else
	{
		::PropVariantInit(&Position);
	}

	const HRESULT Result = SourceReader->SetCurrentPosition(GUID_NULL, Position);

	if (FAILED(Result))
	{
		UE_LOGF(LogMfMedia, Verbose, "Player %p: Failed to set source reader position to %ls: %ls", this, *Time.ToString(), *MfMedia::ResultToString(Result));
		return false;
	}

	CurrentTime = Time;
	Tracks->Restart();

	return true;
}


bool FMfMediaPlayer::InitializePlayer(const TSharedPtr<FArchive, ESPMode::ThreadSafe>& Archive, const FString& Url, bool Precache, const FMediaPlayerOptions* PlayerOptions)
{
	UE_LOGF(LogMfMedia, VeryVerbose, "Player %p: Initializing %ls (archive = %ls, precache = %ls)", this, *Url, Archive.IsValid() ? TEXT("yes") : TEXT("no"), Precache ? TEXT("yes") : TEXT("no"));

	CurrentState = EMediaState::Preparing;

	MediaUrl = Url;
	SourceReaderCallback = new FMfMediaSourceReaderCallback(*this);

	FMediaPlayerOptions LocalPlayerOptions;
	if (PlayerOptions)
	{
		LocalPlayerOptions = *PlayerOptions;
	}

	// initialize presentation on a separate thread
	const EAsyncExecution Execution = Precache ? EAsyncExecution::Thread : EAsyncExecution::ThreadPool;

	Async(Execution, [
		Archive, Url, Precache, LocalPlayerOptions,
		Callback = TComPtr<FMfMediaSourceReaderCallback>(SourceReaderCallback),
		SamplesPtr = TWeakPtr<FMediaSamples, ESPMode::ThreadSafe>(Samples),
		TracksPtr = TWeakPtr<FMfMediaTracks, ESPMode::ThreadSafe>(Tracks)]()
	{
		TSharedPtr<FMediaSamples, ESPMode::ThreadSafe> PinnedSamples = SamplesPtr.Pin();
		TSharedPtr<FMfMediaTracks, ESPMode::ThreadSafe> PinnedTracks= TracksPtr.Pin();

		if (PinnedSamples.IsValid() && PinnedTracks.IsValid())
		{
			TComPtr<IMFMediaSource> MediaSourcePtr = MfMedia::ResolveMediaSource(Archive, Url, Precache);
			PinnedTracks->Initialize(MediaSourcePtr, Callback, PinnedSamples.ToSharedRef(), &LocalPlayerOptions);
		}
	});

	return true;
}


void FMfMediaPlayer::UpdateCharacteristics()
{
	// reset characteristics
	Characteristics = 0u;

	RateControl.Reset();
	RateSupport.Reset();

	ThinnedRates.Empty();
	UnthinnedRates.Empty();

	if (!MediaSource.IsValid())
	{
		return;
	}

	// get characteristics
	HRESULT Result = MediaSource->GetCharacteristics(&Characteristics);

	if (FAILED(Result))
	{
		UE_LOGF(LogMfMedia, Verbose, "Player %p: Failed to get media source characteristics: %ls", this, *MfMedia::ResultToString(Result));
	}

	// get service interface
	TComPtr<IMFGetService> GetService;

	Result = MediaSource->QueryInterface(IID_PPV_ARGS(&GetService));

	if (FAILED(Result))
	{
		UE_LOGF(LogMfMedia, Verbose, "Player %p: Failed to query service interface: %ls", this, *MfMedia::ResultToString(Result));
		return;
	}

	// get rate control & rate support, if available
	Result = GetService->GetService(MF_RATE_CONTROL_SERVICE, IID_PPV_ARGS(&RateControl));

	if (FAILED(Result))
	{
		UE_LOGF(LogMfMedia, Log, "Rate control service unavailable: %ls", *MfMedia::ResultToString(Result));
	}
	else
	{
		UE_LOGF(LogMfMedia, Verbose, "Player %p: Rate control ready", this);
	}

	Result = GetService->GetService(MF_RATE_CONTROL_SERVICE, IID_PPV_ARGS(&RateSupport));

	if (FAILED(Result))
	{
		UE_LOGF(LogMfMedia, Log, "Rate support service unavailable: %ls", *MfMedia::ResultToString(Result));
	}
	else
	{
		UE_LOGF(LogMfMedia, Verbose, "Player %p: Rate support ready", this);
	}

	// cache rate control properties
	if (RateSupport.IsValid())
	{
		// Note: IMFSourceReader does not support reverse playback, even if the
		// media source does, so we are only querying the supported forward rates.

		float MaxRate = 0.0f;
		float MinRate = 0.0f;

		if (SUCCEEDED(RateSupport->GetSlowestRate(MFRATE_FORWARD, TRUE, &MinRate)) &&
			SUCCEEDED(RateSupport->GetFastestRate(MFRATE_FORWARD, TRUE, &MaxRate)))
		{
			ThinnedRates.Add(TRange<float>::Inclusive(MinRate, MaxRate));
		}

		if (SUCCEEDED(RateSupport->GetSlowestRate(MFRATE_FORWARD, FALSE, &MinRate)) &&
			SUCCEEDED(RateSupport->GetFastestRate(MFRATE_FORWARD, FALSE, &MaxRate)))
		{
			UnthinnedRates.Add(TRange<float>::Inclusive(MinRate, MaxRate));
		}
	}
}


/* IMediaControls interface
 *****************************************************************************/

bool FMfMediaPlayer::CanControl(EMediaControl Control) const
{
	if (SourceReader == NULL)
	{
		return false;
	}

	if (Control == EMediaControl::Pause)
	{
		return ((CurrentState == EMediaState::Playing) && ((Characteristics & MFMEDIASOURCE_CAN_PAUSE) != 0));
	}

	if (Control == EMediaControl::Resume)
	{
		return ((CurrentState != EMediaState::Playing) && ThinnedRates.Contains(1.0f));
	}

	if (Control == EMediaControl::Scrub)
	{
		return (((Characteristics & MFMEDIASOURCE_HAS_SLOW_SEEK) == 0) && (ThinnedRates.Contains(0.0f)));
	}

	if (Control == EMediaControl::Seek)
	{
		return (((Characteristics & MFMEDIASOURCE_CAN_SEEK) != 0) && (CurrentDuration > FTimespan::Zero()));
	}

	return false;
}


FTimespan FMfMediaPlayer::GetDuration() const
{
	return CurrentDuration;
}


float FMfMediaPlayer::GetRate() const
{
	return CurrentRate;
}


EMediaState FMfMediaPlayer::GetState() const
{
	return CurrentState;
}


EMediaStatus FMfMediaPlayer::GetStatus() const
{
	return CurrentStatus;
}


TRangeSet<float> FMfMediaPlayer::GetSupportedRates(EMediaRateThinning Thinning) const
{
	return (Thinning == EMediaRateThinning::Thinned) ? ThinnedRates : UnthinnedRates;
}


FTimespan FMfMediaPlayer::GetTime() const
{
	return CurrentTime;
}


bool FMfMediaPlayer::IsLooping() const
{
	return ShouldLoop;
}


bool FMfMediaPlayer::Seek(const FTimespan& Time)
{
	if (SourceReader == NULL)
	{
		return false;
	}

	// validate seek
	if ((CurrentState == EMediaState::Closed) || (CurrentState == EMediaState::Error))
	{
		UE_LOGF(LogMfMedia, Verbose, "Player %p: Cannot seek while closed or in error state", this);
		return false;
	}

	if ((Time < FTimespan::Zero()) || (Time > CurrentDuration))
	{
		UE_LOGF(LogMfMedia, Verbose, "Player %p: Invalid seek time %ls (media duration is %ls)", this, *Time.ToString(), *CurrentDuration.ToString());
		return false;
	}

	UE_LOGF(LogMfMedia, Verbose, "Player %p: Seeking to %ls", this, *Time.ToString());

	if (!CommitTime(Time))
	{
		return false;
	}

	EventSink.ReceiveMediaEvent(EMediaEvent::SeekCompleted);

	return true;
}


bool FMfMediaPlayer::SetLooping(bool Looping)
{
	ShouldLoop = Looping;

	return true;
}


bool FMfMediaPlayer::SetRate(float Rate)
{
	if (SourceReader == NULL)
	{
		return false;
	}

	if (Rate == CurrentRate)
	{
		return true; // rate already set
	}

	if (CurrentDuration == FTimespan::Zero())
	{
		return false; // nothing to play
	}

	BOOL Thin;

	// check whether rate is supported
	if (UnthinnedRates.Contains(Rate))
	{
		UE_LOGF(LogMfMedia, Verbose, "Player %p: Setting rate from to %f to %f (unthinned)", this, CurrentRate, Rate);
		Thin = FALSE;
	}
	else if (ThinnedRates.Contains(Rate))
	{
		UE_LOGF(LogMfMedia, Verbose, "Player %p: Setting rate from to %f to %f (thinned)", this, CurrentRate, Rate);
		Thin = TRUE;
	}
	else
	{
		UE_LOGF(LogMfMedia, Verbose, "Player %p: The rate %f is not supported", this, Rate);
		return false;
	}

	// set play rate
	if (RateControl != NULL)
	{
		if (Rate <= 0.0f)
		{
			// Note: IMFSourceReader does not support reverse playback, even if the
			// media source does. We switch to scrubbing mode and seek instead

			const HRESULT Result = RateControl->SetRate(FALSE, 0.0f);

			if (FAILED(Result))
			{
				UE_LOGF(LogMfMedia, Verbose, "Player %p: Failed to commit rate change from %f to zero: %ls", this, CurrentRate, *MfMedia::ResultToString(Result));
				return false;
			}
		}
		else if (!ThinnedRates.IsEmpty())
		{
			// Note: for those media sources that support thinning in forward play,
			// we update the rate controller

			const TCHAR* ThinnedString = Thin ? TEXT("thinned") : TEXT("unthinned");
			const HRESULT Result = RateControl->SetRate(Thin, Rate);

			if (FAILED(Result))
			{
				UE_LOGF(LogMfMedia, Verbose, "Player %p: Failed to commit rate change from %f to %f [%ls]: %ls", this, CurrentRate, Rate, ThinnedString, *MfMedia::ResultToString(Result));
				return false;
			}
		}
	}

	// handle restarting
	if ((CurrentRate == 0.0f) && (Rate != 0.0f))
	{
		if (CurrentState == EMediaState::Stopped)
		{
			if (Rate < 0.0f)
			{
				CommitTime(CurrentDuration - FTimespan(1));
			}

			PlaybackRestarted = true;
			Tracks->Restart();
		}

		CurrentRate = Rate;
		CurrentState = EMediaState::Playing;

		EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackResumed);

		return true;
	}

	// handle pausing
	if ((CurrentRate != 0.0f) && (Rate == 0.0f))
	{
		CurrentRate = Rate;
		CurrentState = EMediaState::Paused;

		EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackSuspended);

		return true;
	}

	CurrentRate = Rate;

	return true;
}


/* IMfMediaSourceReaderSink interface
 *****************************************************************************/

void FMfMediaPlayer::ReceiveSourceReaderEvent(MediaEventType Event)
{
	switch (Event)
	{
	case MEBufferingStarted:
		CurrentStatus |= EMediaStatus::Buffering;
		EventSink.ReceiveMediaEvent(EMediaEvent::MediaBuffering);
		break;

	case MEBufferingStopped:
		CurrentStatus &= ~EMediaStatus::Buffering;
		EventSink.ReceiveMediaEvent(EMediaEvent::MediaBufferingComplete);
		break;

	case MEConnectEnd:
		CurrentStatus &= ~EMediaStatus::Connecting;
		break;

	case MEConnectStart:
		CurrentStatus |= EMediaStatus::Connecting;
		EventSink.ReceiveMediaEvent(EMediaEvent::MediaConnecting);
		break;

	case MESourceCharacteristicsChanged:
		UpdateCharacteristics();
		break;

	default:
		break; // unsupported event
	}
}


void FMfMediaPlayer::ReceiveSourceReaderFlush()
{
	if (IsInGameThread() || IsInSlateThread())
	{
		Samples->FlushSamples();
	}
	else
	{
		TWeakPtr<FMediaSamples, ESPMode::ThreadSafe> WeakSamples(Samples);
		Async(EAsyncExecution::TaskGraphMainThread, [WeakSamples]() {
			if (auto PinnedSamples = WeakSamples.Pin())
			{
				PinnedSamples->FlushSamples();
			}
			});
	}
}


void FMfMediaPlayer::ReceiveSourceReaderSample(IMFSample* Sample, HRESULT Status, DWORD StreamFlags, DWORD StreamIndex, FTimespan Time)
{
	if ((StreamFlags & MF_SOURCE_READERF_ERROR) != 0)
	{
		SourceReaderError = true;
	}

	Tracks->ProcessSample(Sample, Status, StreamFlags, StreamIndex, Time);
}


#undef LOCTEXT_NAMESPACE

#if PLATFORM_MICROSOFT
	#include "Microsoft/HideMicrosoftPlatformTypes.h"
#endif

#endif //MFMEDIA_SUPPORTED_PLATFORM
