// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MfMediaPrivate.h"

#if MFMEDIA_SUPPORTED_PLATFORM

#include "Containers/UnrealString.h"
#include "IMediaCache.h"
#include "IMediaControls.h"
#include "IMediaPlayer.h"
#include "IMediaView.h"
#include "Misc/Timespan.h"
#include "Templates/SharedPointer.h"
#include "Microsoft/COMPointer.h"

#include "Mf/IMfMediaSourceReaderSink.h"

#if PLATFORM_MICROSOFT
	#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#endif

class FMediaSamples;
class FMfMediaSourceReaderCallback;
class FMfMediaTracks;
class IMediaEventSink;
struct FMediaPlayerOptions;

/**
 * Implements a media player using the Windows Media Foundation framework.
 */
class FMfMediaPlayer
	: public IMediaPlayer
	, protected IMediaCache
	, protected IMediaControls
	, protected IMediaView
	, protected IMfMediaSourceReaderSink
{
public:

	/**
	 * Create and initialize a new instance.
	 *
	 * @param InEventSink The object that receives media events from this player.
	 */
	FMfMediaPlayer(IMediaEventSink& InEventSink);

	/** Virtual destructor. */
	virtual ~FMfMediaPlayer();

public:

	//~ IMediaPlayer interface

	virtual void Close() override;
	virtual IMediaCache& GetCache() override;
	virtual IMediaControls& GetControls() override;
	virtual FString GetInfo() const override;
	virtual FGuid GetPlayerPluginGUID() const override;
	virtual IMediaSamples& GetSamples() override;
	virtual FString GetStats() const override;
	virtual IMediaTracks& GetTracks() override;
	virtual FString GetUrl() const override;
	virtual IMediaView& GetView() override;
	virtual bool Open(const FString& Url, const IMediaOptions* Options) override;
	virtual bool Open(const FString& Url, const IMediaOptions* Options, const FMediaPlayerOptions* PlayerOptions) override;
	virtual bool Open(const TSharedRef<FArchive, ESPMode::ThreadSafe>& Archive, const FString& OriginalUrl, const IMediaOptions* Options) override;
	virtual void TickAudio() override;
	virtual void TickFetch(FTimespan DeltaTime, FTimespan Timecode) override;
	virtual void TickInput(FTimespan DeltaTime, FTimespan Timecode) override;

protected:

	/**
	 * Commit the specified play position.
	 *
	 * @param Time The play position to commit.
	 */
	bool CommitTime(FTimespan Time);

	/**
	 * Initialize the native AvPlayer instance.
	 *
	 * @param Archive The archive being used as a media source (optional).
	 * @param Url The media URL being opened.
	 * @param Precache Whether to precache media into RAM if InURL is a local file.
	 * @return true on success, false otherwise.
	 */
	bool InitializePlayer(const TSharedPtr<FArchive, ESPMode::ThreadSafe>& Archive, const FString& Url, bool Precache, const FMediaPlayerOptions* PlayerOptions);

	/** Get the latest characteristics from the current media source. */
	void UpdateCharacteristics();

protected:

	//~ IMediaControls interface

	virtual bool CanControl(EMediaControl Control) const override;
	virtual FTimespan GetDuration() const override;
	virtual float GetRate() const override;
	virtual EMediaState GetState() const override;
	virtual EMediaStatus GetStatus() const override;
	virtual TRangeSet<float> GetSupportedRates(EMediaRateThinning Thinning) const override;
	virtual FTimespan GetTime() const override;
	virtual bool IsLooping() const override;
	virtual bool Seek(const FTimespan& Time) override;
	virtual bool SetLooping(bool Looping) override;
	virtual bool SetRate(float Rate) override;

protected:

	//~ IMfMediaSourceReaderSink interface

	virtual void ReceiveSourceReaderEvent(MediaEventType Event) override;
	virtual void ReceiveSourceReaderFlush() override;
	virtual void ReceiveSourceReaderSample(IMFSample* Sample, HRESULT Status, DWORD StreamFlags, DWORD StreamIndex, FTimespan Time) override;

private:

	/** Cached media characteristics (capabilities). */
	ULONG Characteristics;

	/** The duration of the currently loaded media. */
	FTimespan CurrentDuration;

	/** The current playback rate. */
	float CurrentRate;

	/** The current playback state. */
	EMediaState CurrentState;

	/** Current status flags. */
	EMediaStatus CurrentStatus;

	/** Current playback time. */
	FTimespan CurrentTime;

	/** The media event handler. */
	IMediaEventSink& EventSink;

	/** The currently opened media. */
	TComPtr<IMFMediaSource> MediaSource;

	/** The URL of the currently opened media. */
	FString MediaUrl;

	/** If playback just restarted from the Stopped state. */
	bool PlaybackRestarted;

	/** The presentation descriptor of the currently opened media. */
	TComPtr<IMFPresentationDescriptor> PresentationDescriptor;

	/** Optional interface for controlling playback rates. */
	TComPtr<IMFRateControl> RateControl;

	/** Optional interface for querying supported playback rates. */
	TComPtr<IMFRateSupport> RateSupport;

	/** Media sample collection. */
	TSharedPtr<FMediaSamples, ESPMode::ThreadSafe> Samples;

	/** Whether playback should loop to the beginning. */
	bool ShouldLoop;

	/** The source reader to use. */
	TComPtr<IMFSourceReader> SourceReader;

	/** Whether an error occurred in the source reader. */
	bool SourceReaderError;

	/** The source reader callback object.*/
	TComPtr<FMfMediaSourceReaderCallback> SourceReaderCallback;

	/** The thinned play rates that the current media session supports. */
	TRangeSet<float> ThinnedRates;

	/** Track collection. */
	TSharedPtr<FMfMediaTracks, ESPMode::ThreadSafe> Tracks;

	/** The unthinned play rates that the current media session supports. */
	TRangeSet<float> UnthinnedRates;
};


#if PLATFORM_MICROSOFT
	#include "Microsoft/HideMicrosoftPlatformTypes.h"
#endif

#endif //MFMEDIA_SUPPORTED_PLATFORM
