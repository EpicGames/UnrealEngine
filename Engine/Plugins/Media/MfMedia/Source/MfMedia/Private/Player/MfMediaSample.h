// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Mf/MfMediaUtils.h"


/**
 * Abstract base class for MF media samples.
 */
class FMfMediaSample
{
protected:

	FMfMediaSample()
		: Duration(FTimespan::Zero())
		, Time(FTimespan::Zero())
	{ }

protected:

	/**
	 * Initialize this sample.
	 *
	 * @param InSample The WMF sample.
	 * @return true on success, false otherwise.
	 */
	bool InitializeSample(IMFSample& InSample)
	{
		// get sample duration
		{
			LONGLONG SampleDuration = 0;
			HRESULT Result = InSample.GetSampleDuration(&SampleDuration);

			if (FAILED(Result))
			{
				UE_LOGF(LogMfMedia, VeryVerbose, "Failed to get sample duration from media sample (%ls)", *MfMedia::ResultToString(Result));
				return false;
			}

			Duration = FTimespan(SampleDuration);
		}

		// get sample flags
		{
			HRESULT Result = InSample.GetSampleFlags(&Flags);

			if (FAILED(Result))
			{
				UE_LOGF(LogMfMedia, VeryVerbose, "Failed to get sample flags from media sample (%ls)", *MfMedia::ResultToString(Result));
				return false;
			}
		}

		// get sample time
		{
			LONGLONG SampleTime = 0;
			HRESULT Result = InSample.GetSampleTime(&SampleTime);

			if (FAILED(Result))
			{
				UE_LOGF(LogMfMedia, VeryVerbose, "Failed to get sample time from media sample (%ls)", *MfMedia::ResultToString(Result));
				return false;
			}

			Time = FTimespan(SampleTime);
		}

		return true;
	}

protected:

	/** Duration for which the sample is valid. */
	FTimespan Duration;

	/** Sample flags. */
	DWORD Flags;

	/** Presentation time for which the sample was generated. */
	FTimespan Time;
};
