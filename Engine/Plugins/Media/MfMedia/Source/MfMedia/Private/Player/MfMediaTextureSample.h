// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MfMediaPrivate.h"

#if MFMEDIA_SUPPORTED_PLATFORM

#include "CoreTypes.h"
#include "IMediaTextureSample.h"
#include "Math/IntPoint.h"
#include "MediaObjectPool.h"
#include "MediaSampleQueue.h"
#include "Misc/Timespan.h"
#include "Microsoft/COMPointer.h"

#include "MfMediaSample.h"


/**
 * Texture sample generated by MfMedia player.
 */
class FMfMediaTextureSample
	: public IMediaTextureSample
	, public IMediaPoolable
	, protected FMfMediaSample
{
public:

	/** Default constructor. */
	FMfMediaTextureSample()
		: Dim(FIntPoint::ZeroValue)
		, OutputDim(FIntPoint::ZeroValue)
		, SampleFormat(EMediaTextureSampleFormat::Undefined)
		, Stride(0)
	{ }

	/** Virtual destructor. */
	virtual ~FMfMediaTextureSample() = default;

public:

	/**
	 * Initialize the sample from a WMF media sample.
	 *
	 * @param InMediaType The sample's media type.
	 * @param InSample The WMF sample.
	 * @param InOutputDim
	 * @param InSlow
	 */
	bool Initialize(IMFMediaType& InMediaType, IMFSample& InSample, const FIntPoint& InBufferDim, uint32 InBufferStride, const FIntPoint& InOutputDim, bool InSlow)
	{
		if (!InitializeSample(InSample))
		{
			return false;
		}

		// get media type
		GUID MajorType;

		if (FAILED(InMediaType.GetGUID(MF_MT_MAJOR_TYPE, &MajorType)) || (MajorType != MFMediaType_Video))
		{
			return false; // not a video sample
		}

		if (FAILED(InMediaType.GetGUID(MF_MT_SUBTYPE, &SubType)))
		{
			return false; // failed to get subtype
		}

		// check interlace mode
		MFVideoInterlaceMode InterlaceMode = (MFVideoInterlaceMode)MFGetAttributeUINT32(&InMediaType, MF_MT_INTERLACE_MODE, MFVideoInterlace_Progressive);

		if ((InterlaceMode != MFVideoInterlace_Progressive) &&
			(InterlaceMode != MFVideoInterlace_MixedInterlaceOrProgressive))
		{
			return false; // @todo gmp: implement support for fielded and mixed interlace modes
		}

#if 0
		// get output frame size
		FIntPoint FrameSize;

		if (FAILED(::MFGetAttributeSize(&InMediaType, MF_MT_FRAME_SIZE, (UINT32*)&FrameSize.X, (UINT32*)&FrameSize.Y)) || (FrameSize.GetMin() == 0))
		{
			return false; // missing or invalid frame size
		}
#endif

		//  set sample format
		if ((SubType == MFVideoFormat_RGB24) || (SubType == MFVideoFormat_RGB32))
		{
			SampleFormat = EMediaTextureSampleFormat::CharBMP;
		}
		else if (SubType == MFVideoFormat_AYUV)
		{
			SampleFormat = EMediaTextureSampleFormat::CharAYUV;
		}
		else if (SubType == MFVideoFormat_UYVY)
		{
			SampleFormat = EMediaTextureSampleFormat::CharUYVY;
		}
		else if (SubType == MFVideoFormat_YUY2)
		{
			SampleFormat = EMediaTextureSampleFormat::CharYUY2;
		}
		else if (SubType == MFVideoFormat_YVYU)
		{
			SampleFormat = EMediaTextureSampleFormat::CharYVYU;
		}
		else if (SubType == MFVideoFormat_NV12)
		{
			SampleFormat = EMediaTextureSampleFormat::CharNV12;
		}
		else
		{
			return false; // unsupported sample format
		}

		OutputDim = InOutputDim;

#if 0
		// calculate buffer dimensions
		if (SampleFormat == EMediaTextureSampleFormat::CharNV12)
		{
			Dim = FIntPoint(FrameSize.X, FrameSize.Y * 3 / 2);
			Stride = Dim.X;
		}
		else if (SampleFormat == EMediaTextureSampleFormat::CharBMP)
		{
			Dim = FrameSize;
			Stride = FrameSize.X * 4;
		}
		else
		{
			Dim = FrameSize;
			Stride = FrameSize.X * 2;
		}
#else
		Dim = InBufferDim;
		Stride = InBufferStride;
#endif

		if (InSlow)
		{
			return InitializeBuffer(InSample);
		}

		Sample = &InSample;
		
		return true;
	}

public:

	//~ IMediaTextureSample interface

	virtual const void* GetBuffer() override
	{
		return Frame.GetData();
	}

	virtual FIntPoint GetDim() const override
	{
		return Dim;
	}

	virtual FTimespan GetDuration() const override
	{
		return Duration;
	}

	virtual EMediaTextureSampleFormat GetFormat() const override
	{
		return SampleFormat;
	}

	virtual FIntPoint GetOutputDim() const override
	{
		return OutputDim;
	}

	virtual uint32 GetStride() const override
	{
		return Stride;
	}

#if WITH_ENGINE
	virtual FRHITexture* GetTexture() const override
	{
		return nullptr;
	}
#endif //WITH_ENGINE

	virtual FMediaTimeStamp GetTime() const override
	{
		return FMediaTimeStamp(Time);
	}

	virtual bool IsCacheable() const override
	{
		return true;
	}

	virtual bool IsOutputSrgb() const override
	{
		return true;
	}

protected:

	bool InitializeBuffer(IMFSample& InSample)
	{
		DWORD NumBuffers;

		if (FAILED(InSample.GetBufferCount(&NumBuffers)) || (NumBuffers == 0))
		{
			return false; // no frame buffers
		}

		TComPtr<IMFMediaBuffer> Buffer;

		if (false)//(NumBuffers == 1)
		{
			if (FAILED(InSample.GetBufferByIndex(0, &Buffer)))
			{
				return false; // failed to get frame buffer
			}
		}
		else
		{
			if (FAILED(InSample.ConvertToContiguousBuffer(&Buffer)))
			{
				return false; // failed to get contiguous buffer
			}
		}

		DWORD BufferSize = 0;
		BYTE* Bytes = NULL;
		LONG SampleStride = 0;

		// lock buffer memory
		TComPtr<IMF2DBuffer> Buffer2D;
/*
		if (SUCCEEDED(Buffer->QueryInterface(IID_PPV_ARGS(&Buffer2D))) &&
			SUCCEEDED(Buffer2D->Lock2D(&Bytes, &SampleStride)))
		{
			BufferSize = SampleStride * OutputDim.Y;
		}
		else*/ if (FAILED(Buffer->Lock(&Bytes, NULL, &BufferSize)))
		{
			return false; // failed to lock buffer
		}

		// Recompute the output buffer vertical dimension - we may get a SMALLER buffer (no padding) here than what we expected before!
		Dim.Y = BufferSize / Stride;

		// copy pixels
		if ((Bytes != NULL) && (BufferSize > 0))
		{
			if (SubType == MFVideoFormat_RGB24)
			{
				int32 FrameSize = BufferSize * 4 / 3;

				Frame.Reset(FrameSize);
				Frame.AddUninitialized(FrameSize);

				uint8* FrameData = Frame.GetData();
				BYTE* PixelsEnd = Bytes + BufferSize;

				while (Bytes < PixelsEnd)
				{
					FrameData[3] = 0;
					FrameData[2] = *Bytes++; // B
					FrameData[1] = *Bytes++; // G
					FrameData[0] = *Bytes++; // R

					FrameData += 4;
				}
			}
			else
			{
				Frame.Reset(BufferSize);
				Frame.Append(Bytes, BufferSize);
			}
		}

		// unlock buffer memory
		if (Buffer2D.IsValid())
		{
			Buffer2D->Unlock2D();
		}
		else
		{
			Buffer->Unlock();
		}
		
		return true;
	}

private:

	/** Width and height of the texture sample. */
	FIntPoint Dim;

	/** The sample's data buffer. */
	TArray<uint8> Frame;

	/** Width and height of the output. */
	FIntPoint OutputDim;

	/** The WMF sample object. */
	TComPtr<IMFSample> Sample;

	/** The sample format. */
	EMediaTextureSampleFormat SampleFormat;

	/** Number of bytes per pixel row. */
	uint32 Stride;

	/** Sub-type of the output media format. */
	GUID SubType;
};


/** Implements a pool for MF texture samples. */
class FMfMediaTextureSamplePool : public TMediaObjectPool<FMfMediaTextureSample> { };


#endif //MFMEDIA_SUPPORTED_PLATFORM
