// Copyright Epic Games, Inc. All Rights Reserved.

#include "MfMediaTracks.h"
#include "MfMediaPrivate.h"

#if MFMEDIA_SUPPORTED_PLATFORM

#include "IMediaTextureSample.h"
#include "Internationalization/Internationalization.h"
#include "Math/IntPoint.h"
#include "MediaHelpers.h"
#include "MediaSampleQueueDepths.h"
#include "MediaSamples.h"
#include "MediaPlayerOptions.h"
#include "Misc/ScopeLock.h"

#include "MfMediaAudioSample.h"
#include "MfMediaTextureSample.h"
#include "Mf/MfMediaUtils.h"

#if PLATFORM_MICROSOFT
	#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#endif

#define MFMEDIATRACKS_TRACE_SAMPLES 0
#if defined(MFMEDIA_PLATFORM_SUPPORTS_ASYNC_TRACK_READER) && MFMEDIA_PLATFORM_SUPPORTS_ASYNC_TRACK_READER !=0
#define MFMEDIATRACKS_USE_ASYNCREADER 1
#else
#define MFMEDIATRACKS_USE_ASYNCREADER 0
#endif
#define MFMEDIATRACKS_USE_DXVA 0			// not implemented yet

#define LOCTEXT_NAMESPACE "FMfMediaSession"


/* FMfMediaSession structors
 *****************************************************************************/

FMfMediaTracks::FMfMediaTracks()
	: AudioDone(true)
	, AudioSamplePending(false)
	, AudioSamplePool(new FMfMediaAudioSamplePool)
	, CaptionDone(true)
	, CaptionSamplePending(false)
	, LastAudioSampleTime(FTimespan::MinValue())
	, LastCaptionSampleTime(FTimespan::MinValue())
	, LastVideoSampleTime(FTimespan::MinValue())
	, MediaSourceChanged(false)
	, SelectedAudioTrack(INDEX_NONE)
	, SelectedCaptionTrack(INDEX_NONE)
	, SelectedVideoTrack(INDEX_NONE)
	, SelectionChanged(false)
	, VideoDone(true)
	, VideoSamplePending(false)
	, VideoSamplePool(new FMfMediaTextureSamplePool)
{ }


FMfMediaTracks::~FMfMediaTracks()
{
	Shutdown();

	delete AudioSamplePool;
	AudioSamplePool = nullptr;

	delete VideoSamplePool;
	VideoSamplePool = nullptr;
}


/* FMfMediaTracks structors
 *****************************************************************************/

void FMfMediaTracks::AppendStats(FString &OutStats) const
{
	FScopeLock Lock(&CriticalSection);

	// audio tracks
	OutStats += TEXT("Audio Tracks\n");

	if (AudioTracks.Num() == 0)
	{
		OutStats += TEXT("\tnone\n");
	}
	else
	{
		for (const FTrack& Track : AudioTracks)
		{
			OutStats += FString::Printf(TEXT("\t%s\n"), *Track.DisplayName.ToString());
			OutStats += TEXT("\t\tNot implemented yet");
		}
	}

	// video tracks
	OutStats += TEXT("Video Tracks\n");

	if (VideoTracks.Num() == 0)
	{
		OutStats += TEXT("\tnone\n");
	}
	else
	{
		for (const FTrack& Track : VideoTracks)
		{
			OutStats += FString::Printf(TEXT("\t%s\n"), *Track.DisplayName.ToString());
			OutStats += TEXT("\t\tNot implemented yet");
		}
	}
}


void FMfMediaTracks::ClearFlags()
{
	FScopeLock Lock(&CriticalSection);

	MediaSourceChanged = false;
	SelectionChanged = false;
}


FTimespan FMfMediaTracks::GetDuration() const
{
	FScopeLock Lock(&CriticalSection);

	if (!SourceReader.IsValid())
	{
		return FTimespan::Zero();
	}

	PROPVARIANT DurationAttrib;
	{
		const HRESULT Result = SourceReader->GetPresentationAttribute(MF_SOURCE_READER_MEDIASOURCE, MF_PD_DURATION, &DurationAttrib);

		if (FAILED(Result))
		{
			return FTimespan::Zero();
		}
	}

	const int64 Duration = (DurationAttrib.vt == VT_UI8) ? (int64)DurationAttrib.uhVal.QuadPart : 0;
	::PropVariantClear(&DurationAttrib);

	return FTimespan(Duration);
}


void FMfMediaTracks::GetFlags(bool& OutMediaSourceChanged, bool& OutSelectionChanged) const
{
	FScopeLock Lock(&CriticalSection);

	OutMediaSourceChanged = MediaSourceChanged;
	OutSelectionChanged = SelectionChanged;
}


IMFMediaSource* FMfMediaTracks::GetMediaSource()
{
	FScopeLock Lock(&CriticalSection);
	return MediaSource;
}


IMFSourceReader* FMfMediaTracks::GetSourceReader()
{
	FScopeLock Lock(&CriticalSection);
	return SourceReader;
}


void FMfMediaTracks::Initialize(IMFMediaSource* InMediaSource, IMFSourceReaderCallback* InSourceReaderCallback, const TSharedRef<FMediaSamples, ESPMode::ThreadSafe>& InSamples, const FMediaPlayerOptions* PlayerOptions)
{
	Shutdown();

	UE_LOGF(LogMfMedia, Verbose, "Tracks: %p: Initializing tracks", this);

	FScopeLock Lock(&CriticalSection);

	MediaSourceChanged = true;

	if (InMediaSource == NULL)
	{
		return;
	}

	// create source reader
	TComPtr<IMFAttributes> Attributes;
	{
		if (FAILED(MFCreateAttributes(&Attributes, 1)))
		{
			UE_LOGF(LogMfMedia, Verbose, "Tracks %p: Failed to create source reader attributes", this);
			return;
		}

		if (FAILED(Attributes->SetUINT32(MF_READWRITE_ENABLE_HARDWARE_TRANSFORMS, TRUE)))
		{
			UE_LOGF(LogMfMedia, Verbose, "Tracks %p: Failed to set one or more source reader attributes", this);
		}

#if MFMEDIATRACKS_USE_DXVA
		if (FAILED(Attributes->SetUINT32(MF_SOURCE_READER_D3D_MANAGER, D3dDeviceManager)) ||
			FAILED(Attributes->SetUINT32(MF_SOURCE_READER_DISABLE_DXVA, FALSE)))
		{
			UE_LOGF(LogMfMedia, Verbose, "Tracks %p: Failed to set DXVA source reader attributes", this);
		}
#endif

#if MFMEDIATRACKS_USE_ASYNCREADER
		if (FAILED(Attributes->SetUnknown(MF_SOURCE_READER_ASYNC_CALLBACK, InSourceReaderCallback)))
		{
			UE_LOGF(LogMfMedia, Verbose, "Tracks %p: Failed to set async callback source reader attribute", this);
		}
#endif
	}

	TComPtr<IMFSourceReader> NewSourceReader;
	{
		const HRESULT Result = ::MFCreateSourceReaderFromMediaSource(InMediaSource, Attributes, &NewSourceReader);

		if (FAILED(Result))
		{
			UE_LOGF(LogMfMedia, Verbose, "Tracks %p: Failed to create source reader: %ls", this, *MfMedia::ResultToString(Result));
			return;
		}
	}

	// initialization successful
	MediaSource = InMediaSource;
	Samples = InSamples;
	SourceReader = NewSourceReader;

	// add streams
	int32 StreamIndex = 0;

	while (AddStreamToTracks(StreamIndex, Info))
	{
		Info += TEXT("\n");
		++StreamIndex;
	}

	UE_LOGF(LogMfMedia, Verbose, "Tracks %p: Found %i streams", this, StreamIndex);

	// Do platform-specific initialization, if required
#if MFMEDIATRACKS_HAS_PLATFORM_INITIALIZE
	PlatformInitialize();
#endif


	Algo::Reverse(AudioTracks);
	Algo::Reverse(CaptionTracks);
	Algo::Reverse(VideoTracks);

	if (PlayerOptions && PlayerOptions->TrackSelection == EMediaPlayerOptionTrackSelectMode::UseTrackOptionIndices)
	{
		// Select tracks based on the options provided
		SelectTrack(EMediaTrackType::Audio, PlayerOptions->Tracks.Audio);
		SelectTrack(EMediaTrackType::Caption, PlayerOptions->Tracks.Caption);
		SelectTrack(EMediaTrackType::Video, PlayerOptions->Tracks.Video);
	}
	else
	{
		// Select default tracks
		SelectTrack(EMediaTrackType::Audio, 0);
		SelectTrack(EMediaTrackType::Caption, INDEX_NONE);
		SelectTrack(EMediaTrackType::Video, 0);
	}
}


void FMfMediaTracks::ProcessSample(IMFSample* Sample, HRESULT Status, DWORD StreamFlags, DWORD StreamIndex, FTimespan Time)
{
	FScopeLock Lock(&CriticalSection);

	if (!Samples.IsValid() || FAILED(Status))
	{
		return;
	}

	if ((StreamFlags & MF_SOURCE_READERF_NATIVEMEDIATYPECHANGED) != 0)
	{
		// @todo gmp: MF3.0 re-initialize source reader
	}

	// process audio sample
	if (SelectedAudioTrack != INDEX_NONE)
	{
		const FTrack& Track = AudioTracks[SelectedAudioTrack];

		if (Track.StreamIndex == StreamIndex)
		{
			if (Samples->NumAudioSamples() >= FMediaPlayerQueueDepths::MaxAudioSinkDepth)
			{
				return;
			}

			if (((StreamFlags & MF_SOURCE_READERF_ENDOFSTREAM) != 0) || ((StreamFlags & MF_SOURCE_READERF_ERROR) != 0))
			{
				UE_LOGF(LogMfMedia, Verbose, "Tracks %p: Audio done", this);

				AudioDone = true;
			}

			if ((Track.SelectedFormat != INDEX_NONE) && (Sample != NULL))
			{
				const FFormat& Format = Track.Formats[Track.SelectedFormat];
				check(Format.OutputType.IsValid());

				const TSharedRef<FMfMediaAudioSample, ESPMode::ThreadSafe> AudioSample = AudioSamplePool->AcquireShared();

				if (AudioSample->Initialize(*Format.OutputType, *Sample, Format.Audio.NumChannels, Format.Audio.SampleRate))
				{
					Samples->AddAudio(AudioSample);
					LastAudioSampleTime = AudioSample->GetTime().Time;

					#if MFMEDIATRACKS_TRACE_SAMPLES
						UE_LOGF(LogMfMedia, VeryVerbose, "Tracks %p: Audio sample processed: %ls", this, *LastAudioSampleTime.ToString());
					#endif
				}
			}

			AudioSamplePending = false;
			UpdateAudio();

			return;
		}
	}

	// process caption sample
	if (SelectedCaptionTrack != INDEX_NONE)
	{
		const FTrack& Track = CaptionTracks[SelectedCaptionTrack];

		if (Track.StreamIndex == StreamIndex)
		{
			if (Samples->NumCaptionSamples() >= FMediaPlayerQueueDepths::MaxCaptionSinkDepth)
			{
				return;
			}

			if (((StreamFlags & MF_SOURCE_READERF_ENDOFSTREAM) != 0) || ((StreamFlags & MF_SOURCE_READERF_ERROR) != 0))
			{
				UE_LOGF(LogMfMedia, Verbose, "Tracks %p: Caption done", this);

				CaptionDone = true;
			}

			if ((Track.SelectedFormat != INDEX_NONE) && (Sample != NULL))
			{
				const FFormat& Format = Track.Formats[Track.SelectedFormat];
				check(Format.OutputType.IsValid());
/*
				const auto CaptionSample = MakeShared<FMfMediaOverlaySample, ESPMode::ThreadSafe>();

				if (CaptionSample->Initialize(*Format.OutputType, *Sample))
				{
					Samples->AddCaption(CaptionSample);
					LastCaptionSampleTime = CaptionSample->GetTime();
*/
					LONGLONG SampleTime = 0;
					Sample->GetSampleTime(&SampleTime);
					LastCaptionSampleTime = FTimespan(SampleTime);

					#if MFMEDIATRACKS_TRACE_SAMPLES
						UE_LOGF(LogMfMedia, VeryVerbose, "Tracks %p: Caption sample processed: %ls", this, *LastCaptionSampleTime.ToString());
					#endif
//				}
			}

			CaptionSamplePending = false;
			UpdateCaptions();
		}
	}

	// process video sample
	if (SelectedVideoTrack != INDEX_NONE)
	{
		const FTrack& Track = VideoTracks[SelectedVideoTrack];

		if (Track.StreamIndex == StreamIndex)
		{
			if (Samples->NumVideoSamples() >= FMediaPlayerQueueDepths::MaxVideoSinkDepth)
			{
				return;
			}

			if (((StreamFlags & MF_SOURCE_READERF_ENDOFSTREAM) != 0) || ((StreamFlags & MF_SOURCE_READERF_ERROR) != 0))
			{
				UE_LOGF(LogMfMedia, Verbose, "Tracks %p: Video done", this);

				VideoDone = true;
			}

			if ((Track.SelectedFormat != INDEX_NONE) && (Sample != NULL))
			{
				const FFormat& Format = Track.Formats[Track.SelectedFormat];
				check(Format.OutputType.IsValid());

				const TSharedRef<FMfMediaTextureSample, ESPMode::ThreadSafe> VideoSample = VideoSamplePool->AcquireShared();

				if (VideoSample->Initialize(*Format.OutputType, *Sample, Format.Video.BufferDim, Format.Video.BufferStride, Format.Video.OutputDim, true))
				{
					Samples->AddVideo(VideoSample);
					LastVideoSampleTime = VideoSample->GetTime().Time;

					#if MFMEDIATRACKS_TRACE_SAMPLES
						UE_LOGF(LogMfMedia, VeryVerbose, "Tracks %p: Video sample processed: %ls", this, *LastVideoSampleTime.ToString());
					#endif
				}
			}

			VideoSamplePending = false;
			UpdateVideo();

			return;
		}
	}
}


void FMfMediaTracks::Restart()
{
	UE_LOGF(LogMfMedia, Verbose, "Tracks %p: Restarting sample processing", this);

	FScopeLock Lock(&CriticalSection);

	if (SourceReader.IsValid())
	{
//		SourceReader->Flush(MF_SOURCE_READER_ALL_STREAMS);
	}

	AudioDone = (AudioTracks.Num() == 0);
	CaptionDone = (CaptionTracks.Num() == 0);
	VideoDone = (VideoTracks.Num() == 0);

	AudioSampleRange = TRange<FTimespan>::Empty();
	CaptionSampleRange = TRange<FTimespan>::Empty();
	VideoSampleRange = TRange<FTimespan>::Empty();

	AudioSamplePending = false;
	CaptionSamplePending = false;
	VideoSamplePending = false;

	LastAudioSampleTime = FTimespan::MinValue();
	LastCaptionSampleTime = FTimespan::MinValue();
	LastVideoSampleTime = FTimespan::MinValue();
}


void FMfMediaTracks::Shutdown()
{
	FScopeLock Lock(&CriticalSection);

	AudioSamplePool->Reset();
	VideoSamplePool->Reset();

	SelectedAudioTrack = INDEX_NONE;
	SelectedCaptionTrack = INDEX_NONE;
	SelectedVideoTrack = INDEX_NONE;

	AudioTracks.Empty();
	CaptionTracks.Empty();
	VideoTracks.Empty();

	Info.Empty();

	if (MediaSource != NULL)
	{
		MediaSource->Shutdown();
		MediaSource = NULL;
	}

	AudioDone = true;
	CaptionDone = true;
	VideoDone = true;

	Samples.Reset();
	SourceReader.Reset();

	MediaSourceChanged = false;
}


void FMfMediaTracks::TickAudio(float Rate, FTimespan Time)
{
	FScopeLock Lock(&CriticalSection);

	if ((Rate <= 0.0f) || (Rate > 2.0f))
	{
		return; // no audio in reverse or very fast forward
	}

	AudioSampleRange = TRange<FTimespan>::AtMost(Time + FTimespan::FromSeconds(Rate));

	UpdateAudio();
}


void FMfMediaTracks::TickInput(float Rate, FTimespan Time)
{
	FScopeLock Lock(&CriticalSection);

	if (Rate > 0.0f)
	{
		CaptionSampleRange = TRange<FTimespan>::AtMost(Time);
	}
	else if (Rate < 0.0f)
	{
		CaptionSampleRange = TRange<FTimespan>::AtLeast(Time);
	}
	else
	{
		// reuse previous range
	}

	VideoSampleRange = CaptionSampleRange;

	UpdateCaptions();
	UpdateVideo();
}


/* IMediaTracks interface
 *****************************************************************************/

bool FMfMediaTracks::GetAudioTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaAudioTrackFormat& OutFormat) const
{
	FScopeLock Lock(&CriticalSection);

	const FFormat* Format = GetAudioFormat(TrackIndex, FormatIndex);

	if (Format == nullptr)
	{
		return false;
	}

	OutFormat.BitsPerSample = Format->Audio.BitsPerSample;
	OutFormat.NumChannels = Format->Audio.NumChannels;
	OutFormat.SampleRate = Format->Audio.SampleRate;
	OutFormat.TypeName = Format->TypeName;

	return true;
}


int32 FMfMediaTracks::GetNumTracks(EMediaTrackType TrackType) const
{
	FScopeLock Lock(&CriticalSection);

	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		return AudioTracks.Num();

	case EMediaTrackType::Caption:
		return CaptionTracks.Num();

	case EMediaTrackType::Video:
		return VideoTracks.Num();

	default:
		break; // unsupported track type
	}

	return 0;
}


int32 FMfMediaTracks::GetNumTrackFormats(EMediaTrackType TrackType, int32 TrackIndex) const
{
	FScopeLock Lock(&CriticalSection);

	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		if (AudioTracks.IsValidIndex(TrackIndex))
		{
			return AudioTracks[TrackIndex].Formats.Num();
		}

	case EMediaTrackType::Caption:
		if (CaptionTracks.IsValidIndex(TrackIndex))
		{
			return 1;
		}

	case EMediaTrackType::Video:
		if (VideoTracks.IsValidIndex(TrackIndex))
		{
			return VideoTracks[TrackIndex].Formats.Num();
		}

	default:
		break; // unsupported track type
	}

	return 0;
}


int32 FMfMediaTracks::GetSelectedTrack(EMediaTrackType TrackType) const
{
	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		return SelectedAudioTrack;

	case EMediaTrackType::Caption:
		return SelectedCaptionTrack;

	case EMediaTrackType::Video:
		return SelectedVideoTrack;

	default:
		break; // unsupported track type
	}

	return INDEX_NONE;
}


FText FMfMediaTracks::GetTrackDisplayName(EMediaTrackType TrackType, int32 TrackIndex) const
{
	FScopeLock Lock(&CriticalSection);

	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		if (AudioTracks.IsValidIndex(TrackIndex))
		{
			return AudioTracks[TrackIndex].DisplayName;
		}
		break;

	case EMediaTrackType::Caption:
		if (CaptionTracks.IsValidIndex(TrackIndex))
		{
			return CaptionTracks[TrackIndex].DisplayName;
		}
		break;

	case EMediaTrackType::Video:
		if (VideoTracks.IsValidIndex(TrackIndex))
		{
			return VideoTracks[TrackIndex].DisplayName;
		}
		break;

	default:
		break; // unsupported track type
	}

	return FText::GetEmpty();
}


int32 FMfMediaTracks::GetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex) const
{
	FScopeLock Lock(&CriticalSection);

	const FTrack* Track = GetTrack(TrackType, TrackIndex);
	return (Track != nullptr) ? Track->SelectedFormat : INDEX_NONE;
}


FString FMfMediaTracks::GetTrackLanguage(EMediaTrackType TrackType, int32 TrackIndex) const
{
	FScopeLock Lock(&CriticalSection);

	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		if (AudioTracks.IsValidIndex(TrackIndex))
		{
			return AudioTracks[TrackIndex].Language;
		}
		break;

	case EMediaTrackType::Caption:
		if (CaptionTracks.IsValidIndex(TrackIndex))
		{
			return CaptionTracks[TrackIndex].Language;
		}
		break;

	case EMediaTrackType::Video:
		if (VideoTracks.IsValidIndex(TrackIndex))
		{
			return VideoTracks[TrackIndex].Language;
		}
		break;

	default:
		break; // unsupported track type
	}

	return FString();
}


FString FMfMediaTracks::GetTrackName(EMediaTrackType TrackType, int32 TrackIndex) const
{
	FScopeLock Lock(&CriticalSection);

	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		if (AudioTracks.IsValidIndex(TrackIndex))
		{
			return AudioTracks[TrackIndex].Name;
		}
		break;

	case EMediaTrackType::Caption:
		if (CaptionTracks.IsValidIndex(TrackIndex))
		{
			return CaptionTracks[TrackIndex].Name;
		}
		break;

	case EMediaTrackType::Video:
		if (VideoTracks.IsValidIndex(TrackIndex))
		{
			return VideoTracks[TrackIndex].Name;
		}
		break;

	default:
		break; // unsupported track type
	}

	return FString();
}


bool FMfMediaTracks::GetVideoTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaVideoTrackFormat& OutFormat) const
{
	FScopeLock Lock(&CriticalSection);

	const FFormat* Format = GetVideoFormat(TrackIndex, FormatIndex);

	if (Format == nullptr)
	{
		return false;
	}

	OutFormat.Dim = Format->Video.OutputDim;
	OutFormat.FrameRate = Format->Video.FrameRate;
	OutFormat.FrameRates = Format->Video.FrameRates;
	OutFormat.TypeName = Format->TypeName;

	return true;
}


bool FMfMediaTracks::SelectTrack(EMediaTrackType TrackType, int32 TrackIndex)
{
	if (SourceReader == nullptr)
	{
		return false; // not initialized
	}

	UE_LOGF(LogMfMedia, Verbose, "Session %p: Selecting %ls track %i", this, *MediaUtils::TrackTypeToString(TrackType), TrackIndex);

	FScopeLock Lock(&CriticalSection);

	int32* SelectedTrack = nullptr;
	TArray<FTrack>* Tracks = nullptr;

	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		SelectedTrack = &SelectedAudioTrack;
		Tracks = &AudioTracks;
		break;

	case EMediaTrackType::Caption:
		SelectedTrack = &SelectedCaptionTrack;
		Tracks = &CaptionTracks;
		break;

	case EMediaTrackType::Video:
		SelectedTrack = &SelectedVideoTrack;
		Tracks = &VideoTracks;
		break;

	default:
		return false;
	}

	check(SelectedTrack != nullptr);
	check(Tracks != nullptr);

	if (TrackIndex == *SelectedTrack)
	{
		return true; // already selected
	}

	if ((TrackIndex != INDEX_NONE) && !Tracks->IsValidIndex(TrackIndex))
	{
		return false; // invalid track
	}

	// deselect stream for old track
	if (*SelectedTrack != INDEX_NONE)
	{
		const DWORD StreamIndex = (*Tracks)[*SelectedTrack].StreamIndex;
		{
			const HRESULT Result = SourceReader->SetStreamSelection(StreamIndex, FALSE);

			if (FAILED(Result))
			{
				UE_LOGF(LogMfMedia, Verbose, "Tracks %p: Failed to deselect stream %i on source reader: %ls", this, StreamIndex, *MfMedia::ResultToString(Result));
				return false;
			}
		}

		UE_LOGF(LogMfMedia, Verbose, "Tracks %p: Disabled stream %i", this, StreamIndex);

		*SelectedTrack = INDEX_NONE;
		SelectionChanged = true;

		HRESULT Result = SourceReader->Flush(StreamIndex);

		if (FAILED(Result))
		{
			UE_LOGF(LogMfMedia, Verbose, "Tracks %p: Failed to flush deselected stream %i: %ls", this, StreamIndex, *MfMedia::ResultToString(Result));
		}
	}

	// select stream for new track
	if (TrackIndex != INDEX_NONE)
	{
		const DWORD StreamIndex = (*Tracks)[TrackIndex].StreamIndex;
		const HRESULT Result = SourceReader->SetStreamSelection(StreamIndex, TRUE);

		if (FAILED(Result))
		{
			UE_LOGF(LogMfMedia, Verbose, "Tracks %p: Failed to select stream %i on source reader: %ls", this, StreamIndex, *MfMedia::ResultToString(Result));
			return false;
		}

		UE_LOGF(LogMfMedia, Verbose, "Tracks %p: Enabled stream %i", this, StreamIndex);

		*SelectedTrack = TrackIndex;
		SelectionChanged = true;
	}

	return true;
}


bool FMfMediaTracks::SetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex, int32 FormatIndex)
{
	FScopeLock Lock(&CriticalSection);

	TArray<FTrack>* Tracks = nullptr;

	// get track
	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		Tracks = &AudioTracks;
		break;

	case EMediaTrackType::Caption:
		Tracks = &CaptionTracks;
		break;

	case EMediaTrackType::Video:
		Tracks = &VideoTracks;
		break;

	default:
		return false; // unsupported track type
	};

	check(Tracks != nullptr);

	if (!Tracks->IsValidIndex(TrackIndex))
	{
		return false; // invalid track
	}

	FTrack& Track = (*Tracks)[TrackIndex];

	if (Track.SelectedFormat == FormatIndex)
	{
		return true; // format already set
	}

	if (!Track.Formats.IsValidIndex(FormatIndex))
	{
		return false; // format not found
	}

	// get selected format
	const FFormat& Format = Track.Formats[FormatIndex];

	check(Format.InputType.IsValid());
	check(Format.OutputType.IsValid());

	// set track format
	UE_LOGF(LogMfMedia, Verbose, "Tracks %p: Setting format %i instead of %i on %ls track %i (%i formats)", this, FormatIndex, Track.SelectedFormat, *MediaUtils::TrackTypeToString(TrackType), TrackIndex, Track.Formats.Num());
	{
		HRESULT Result = SourceReader->SetCurrentMediaType(Track.StreamIndex, NULL, Format.OutputType);

		if (FAILED(Result))
		{
			UE_LOGF(LogMfMedia, Verbose, "Tracks %p: Failed to set selected media type on reader for stream %i: %ls", this, Track.StreamIndex, *MfMedia::ResultToString(Result));
			return false;
		}
	}

	Track.SelectedFormat = FormatIndex;
	SelectionChanged = true;

	return true;
}


/* FMfMediaTracks implementation
 *****************************************************************************/

bool FMfMediaTracks::AddStreamToTracks(uint32 StreamIndex, FString& OutInfo)
{
	OutInfo += FString::Printf(TEXT("Stream %i\n"), StreamIndex);

	// get current format
	TComPtr<IMFMediaType> CurrentMediaType;
	{
		const HRESULT Result = SourceReader->GetCurrentMediaType(StreamIndex, &CurrentMediaType);

		if (FAILED(Result))
		{
			UE_LOGF(LogMfMedia, Verbose, "Tracks %p: Failed to get current media type in stream %i: %ls", this, StreamIndex, *MfMedia::ResultToString(Result));
			OutInfo += TEXT("\tfailed to get current format\n");

			return false;
		}
	}

	// skip unsupported handler types
	GUID MajorType;
	{
		const HRESULT Result = CurrentMediaType->GetGUID(MF_MT_MAJOR_TYPE, &MajorType);

		if (FAILED(Result))
		{
			UE_LOGF(LogMfMedia, Verbose, "Tracks %p: Failed to determine major type of stream %i: %ls", this, StreamIndex, *MfMedia::ResultToString(Result));
			OutInfo += TEXT("\tfailed to determine MajorType\n");

			return false;
		}

		UE_LOGF(LogMfMedia, Verbose, "Tracks %p: Major type of stream %i is %ls", this, StreamIndex, *MfMedia::MajorTypeToString(MajorType));
		OutInfo += FString::Printf(TEXT("\tType: %s\n"), *MfMedia::MajorTypeToString(MajorType));

		if ((MajorType != MFMediaType_Audio) &&
			(MajorType != MFMediaType_SAMI) &&
			(MajorType != MFMediaType_Video))
		{
			UE_LOGF(LogMfMedia, Verbose, "Tracks %p: Unsupported major type %ls of stream %i", this, *MfMedia::MajorTypeToString(MajorType), StreamIndex);
			OutInfo += TEXT("\tMajorType is not supported\n");

			return false;
		}
	}

	// @todo gmp: handle protected content
	BOOL Protected = FALSE;
	{
		PROPVARIANT ProtectedAttrib;

		if (SUCCEEDED(SourceReader->GetPresentationAttribute(StreamIndex, MF_SD_PROTECTED, &ProtectedAttrib)))
		{
			Protected = (ProtectedAttrib.vt == VT_BOOL) && (ProtectedAttrib.boolVal == VARIANT_TRUE);
			::PropVariantClear(&ProtectedAttrib);

			if (Protected == TRUE)
			{
				OutInfo += FString::Printf(TEXT("\tProtected content\n"));
			}
		}
	}

	// create & add track
	FTrack* Track = nullptr;

	if (MajorType == MFMediaType_Audio)
	{
		const int32 TrackIndex = AudioTracks.AddDefaulted();
		Track = &AudioTracks[TrackIndex];
	}
	else if (MajorType == MFMediaType_SAMI)
	{
		const int32 TrackIndex = CaptionTracks.AddDefaulted();
		Track = &CaptionTracks[TrackIndex];
	}
	else if (MajorType == MFMediaType_Video)
	{
		const int32 TrackIndex = VideoTracks.AddDefaulted();
		Track = &VideoTracks[TrackIndex];
	}

	check(Track != nullptr);
	Track->SelectedFormat = INDEX_NONE;

	// add track formats
	const bool AllowNonStandardCodecs = false;// GetDefault<UMfMediaSettings>()->AllowNonStandardCodecs;

	int32 TypeIndex = INDEX_NONE;

	while (true)
	{
		++TypeIndex;

		// get media type
		TComPtr<IMFMediaType> MediaType;
		{
			const HRESULT Result = SourceReader->GetNativeMediaType(StreamIndex, TypeIndex, &MediaType);

			if (Result == MF_E_NO_MORE_TYPES)
			{
				break; // done
			}

			OutInfo += FString::Printf(TEXT("\tFormat %i\n"), TypeIndex);

			if (FAILED(Result))
			{
				OutInfo += TEXT("\t\tfailed to get media type\n");

				continue;
			}
		}

		// get sub-type
		GUID SubType;

		if (MajorType == MFMediaType_SAMI)
		{
			FMemory::Memzero(SubType);
		}
		else
		{
			const HRESULT Result = MediaType->GetGUID(MF_MT_SUBTYPE, &SubType);

			if (FAILED(Result))
			{
				UE_LOGF(LogMfMedia, Verbose, "Tracks %p: Failed to get sub-type of format %i in stream %i: %ls", this, TypeIndex, StreamIndex, *MfMedia::ResultToString(Result));
				OutInfo += TEXT("\t\tfailed to get sub-type\n");

				continue;;
			}
		}

		const FString TypeName = MfMedia::SubTypeToString(SubType);
		OutInfo += FString::Printf(TEXT("\t\tCodec: %s\n"), *TypeName);

		// create output type
		TComPtr<IMFMediaType> OutputType = MfMedia::CreateOutputType(MajorType, SubType, AllowNonStandardCodecs);

		if (!OutputType.IsValid())
		{
			OutInfo += TEXT("\t\tfailed to create output type\n");

			continue;
		}

		// add format details
		int32 FormatIndex = INDEX_NONE;

		if (MajorType == MFMediaType_Audio)
		{
			const uint32 BitsPerSample = ::MFGetAttributeUINT32(MediaType, MF_MT_AUDIO_BITS_PER_SAMPLE, 16);
			const uint32 NumChannels = ::MFGetAttributeUINT32(MediaType, MF_MT_AUDIO_NUM_CHANNELS, 0);
			const uint32 SampleRate = ::MFGetAttributeUINT32(MediaType, MF_MT_AUDIO_SAMPLES_PER_SECOND, 0);

			FormatIndex = Track->Formats.Add({
				MediaType,
				OutputType,
				TypeName,
				{
					BitsPerSample,
					NumChannels,
					SampleRate
				},
				{ 0 }
			});

			OutInfo += FString::Printf(TEXT("\t\tChannels: %i\n"), NumChannels);
			OutInfo += FString::Printf(TEXT("\t\tSample Rate: %i Hz\n"), SampleRate);
			OutInfo += FString::Printf(TEXT("\t\tBits Per Sample: %i\n"), BitsPerSample);
		}
		else if (MajorType == MFMediaType_SAMI)
		{
			FormatIndex = Track->Formats.Add({
				MediaType,
				OutputType,
				TypeName,
				{ 0 },
				{ 0 }
			});
		}
		else if (MajorType == MFMediaType_Video)
		{
			GUID OutputSubType;
			{
				const HRESULT Result = OutputType->GetGUID(MF_MT_SUBTYPE, &OutputSubType);

				if (FAILED(Result))
				{
					UE_LOGF(LogMfMedia, Verbose, "Tracks %p: Failed to get video output sub-type for stream %i: %ls", this, StreamIndex, *MfMedia::ResultToString(Result));
					OutInfo += FString::Printf(TEXT("\t\tfailed to get sub-type"));

					continue;
				}
			}

			const uint32 BitRate = ::MFGetAttributeUINT32(MediaType, MF_MT_AVG_BITRATE, 0);

			FIntPoint OutputDim;
			{
				if (SUCCEEDED(::MFGetAttributeSize(MediaType, MF_MT_FRAME_SIZE, (UINT32*)&OutputDim.X, (UINT32*)&OutputDim.Y)))
				{
					OutInfo += FString::Printf(TEXT("\t\tDimensions: %i x %i\n"), OutputDim.X, OutputDim.Y);
				}
				else
				{
					OutputDim = FIntPoint::ZeroValue;
					OutInfo += FString::Printf(TEXT("\t\tDimensions: n/a\n"));
				}
			}

			float FrameRate;
			{
				UINT32 Numerator = 0;
				UINT32 Denominator = 1;

				if (SUCCEEDED(::MFGetAttributeRatio(MediaType, MF_MT_FRAME_RATE, &Numerator, &Denominator)))
				{
					FrameRate = static_cast<float>(Numerator) / Denominator;
					OutInfo += FString::Printf(TEXT("\t\tFrame Rate: %g fps\n"), FrameRate);
				}
				else
				{
					FrameRate = 0.0f;
					OutInfo += FString::Printf(TEXT("\t\tFrame Rate: n/a\n"));
				}
			}

			TRange<float> FrameRates;
			{
				UINT32 Numerator = 0;
				UINT32 Denominator = 1;
				float Min = -1.0f;
				float Max = -1.0f;

				if (SUCCEEDED(::MFGetAttributeRatio(MediaType, MF_MT_FRAME_RATE_RANGE_MIN, &Numerator, &Denominator)))
				{
					Min = static_cast<float>(Numerator) / Denominator;
				}

				if (SUCCEEDED(::MFGetAttributeRatio(MediaType, MF_MT_FRAME_RATE_RANGE_MAX, &Numerator, &Denominator)))
				{
					Max = static_cast<float>(Numerator) / Denominator;
				}

				if ((Min >= 0.0f) && (Max >= 0.0f))
				{
					FrameRates = TRange<float>::Inclusive(Min, Max);
				}
				else
				{
					FrameRates = TRange<float>(FrameRate);
				}

				OutInfo += FString::Printf(TEXT("\t\tFrame Rate Range: %g - %g fps\n"), FrameRates.GetLowerBoundValue(), FrameRates.GetUpperBoundValue());

				if (FrameRates.IsDegenerate() && (FrameRates.GetLowerBoundValue() == 1.0f))
				{
					OutInfo += FString::Printf(TEXT("\t\tpossibly a still image stream (may not work)\n"));
				}
			}

			FIntPoint BufferDim;
			uint32 BufferStride;
			EMediaTextureSampleFormat SampleFormat;
			{
#if PLATFORM_WINDOWS
				if (OutputSubType == MFVideoFormat_NV12)
#endif
				{
					BufferDim = FIntPoint(Align(OutputDim.X, 16), Align(OutputDim.Y, 16) * 3 / 2);
					BufferStride = BufferDim.X;
					SampleFormat = EMediaTextureSampleFormat::CharNV12;
				}
#if PLATFORM_WINDOWS
				else if (OutputSubType == MFVideoFormat_RGB32)
				{
					BufferDim = OutputDim;
					BufferStride = OutputDim.X * 4;
					SampleFormat = EMediaTextureSampleFormat::CharBMP;
				}
				else
				{
					int32 AlignedOutputX = OutputDim.X;

					if ((SubType == MFVideoFormat_H264) || (SubType == MFVideoFormat_H264_ES))
					{
						AlignedOutputX = Align(AlignedOutputX, 16);
					}

					int32 SampleStride = AlignedOutputX * 2; // 2 bytes per pixel

					if (SampleStride < 0)
					{
						SampleStride = -SampleStride;
					}

					BufferDim = FIntPoint(AlignedOutputX / 2, OutputDim.Y); // 2 pixels per texel
					BufferStride = SampleStride;
					SampleFormat = EMediaTextureSampleFormat::CharYUY2;
				}
#endif //PLATFORM_WINDOWS
			}

#if PLATFORM_WINDOWS
			GUID FormatType = GUID_NULL;

			// prevent duplicates for legacy DirectShow media types
			// see: https://msdn.microsoft.com/en-us/library/windows/desktop/ff485858(v=vs.85).aspx

			if (SUCCEEDED(MediaType->GetGUID(MF_MT_AM_FORMAT_TYPE, &FormatType)))
			{
				if (FormatType == FORMAT_VideoInfo)
				{
					for (int32 Index = Track->Formats.Num() - 1; Index >= 0; --Index)
					{
						const FFormat& Format = Track->Formats[Index];

						if ((Format.Video.FormatType == FORMAT_VideoInfo2) &&
							(Format.Video.FrameRates == FrameRates) &&
							(Format.Video.OutputDim == OutputDim) &&
							(Format.TypeName == TypeName))
						{
							FormatIndex = Index; // keep newer format

							break;
						}
					}
				}
				else if (FormatType == FORMAT_VideoInfo2)
				{
					for (int32 Index = Track->Formats.Num() - 1; Index >= 0; --Index)
					{
						FFormat& Format = Track->Formats[Index];

						if ((Format.Video.FormatType == FORMAT_VideoInfo) &&
							(Format.Video.FrameRates == FrameRates) &&
							(Format.Video.OutputDim == OutputDim) &&
							(Format.TypeName == TypeName))
						{
							Format.InputType = MediaType; // replace legacy format
							FormatIndex = Index;

							break;
						}
					}
				}
			}

			if (FormatIndex == INDEX_NONE)
			{
				FormatIndex = Track->Formats.Add({
					MediaType,
					OutputType,
					TypeName,
					{ 0 },
					{
						BitRate,
						BufferDim,
						BufferStride,
						FormatType,
						FrameRate,
						FrameRates,
						OutputDim,
						SampleFormat
					}
				});
			}
#endif //PLATFORM_WINDOWS
		}
		else
		{
			check(false); // should never get here
		}

		if (MediaType == CurrentMediaType)
		{
			const HRESULT Result = SourceReader->SetCurrentMediaType(StreamIndex, NULL, OutputType);

			if (SUCCEEDED(Result))
			{
				Track->SelectedFormat = FormatIndex;
			}
			else
			{
				UE_LOGF(LogMfMedia, Verbose, "Tracks %p: Failed to set current media type on reader for stream %i: %ls", this, Track->StreamIndex, *MfMedia::ResultToString(Result));
			}
		}
	}

	// ensure that a track format is selected
	if (Track->SelectedFormat == INDEX_NONE)
	{
		for (int32 FormatIndex = 0; FormatIndex < Track->Formats.Num(); ++FormatIndex)
		{
			const FFormat& Format = Track->Formats[FormatIndex];
			const HRESULT Result = SourceReader->SetCurrentMediaType(StreamIndex, NULL, Format.OutputType);

			if (SUCCEEDED(Result))
			{
				UE_LOGF(LogMfMedia, Verbose, "Tracks %p: Picked default format %i for stream %i", this, FormatIndex, StreamIndex);
				Track->SelectedFormat = FormatIndex;
				break;
			}
		}

		if (Track->SelectedFormat == INDEX_NONE)
		{
			UE_LOGF(LogMfMedia, Verbose, "Tracks %p: No supported media types found in stream %i", this, StreamIndex);
			OutInfo += TEXT("\tunsupported media type\n");
		}
	}

	// set track details
	PROPVARIANT LanguageAttrib;

	if (SUCCEEDED(SourceReader->GetPresentationAttribute(StreamIndex, MF_SD_LANGUAGE, &LanguageAttrib)))
	{
		if ((LanguageAttrib.vt == VT_LPWSTR) || (LanguageAttrib.vt == VT_BSTR))
		{
			if (LanguageAttrib.pwszVal != NULL)
			{
				Track->Language = LanguageAttrib.pwszVal;
			}
		}

		::PropVariantClear(&LanguageAttrib);
	}

	PROPVARIANT NameAttrib;

	if (SUCCEEDED(SourceReader->GetPresentationAttribute(StreamIndex, MF_SD_STREAM_NAME, &NameAttrib)))
	{
		if ((NameAttrib.vt == VT_LPWSTR) || (NameAttrib.vt == VT_BSTR))
		{
			if (NameAttrib.pwszVal != NULL)
			{
				Track->Name = NameAttrib.pwszVal;
			}
		}

		::PropVariantClear(&NameAttrib);
	}

	Track->DisplayName = (Track->Name.IsEmpty())
		? FText::Format(LOCTEXT("UnnamedStreamFormat", "Unnamed Track (Stream {0})"), FText::AsNumber((uint32)StreamIndex))
		: FText::FromString(Track->Name);

	Track->Protected = (Protected == TRUE);
	Track->StreamIndex = StreamIndex;

	return true;
}


const FMfMediaTracks::FFormat* FMfMediaTracks::GetAudioFormat(int32 TrackIndex, int32 FormatIndex) const
{
	if (AudioTracks.IsValidIndex(TrackIndex))
	{
		const FTrack& Track = AudioTracks[TrackIndex];

		if (Track.Formats.IsValidIndex(FormatIndex))
		{
			return &Track.Formats[FormatIndex];
		}
	}

	return nullptr;
}


const FMfMediaTracks::FTrack* FMfMediaTracks::GetTrack(EMediaTrackType TrackType, int32 TrackIndex) const
{
	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		if (AudioTracks.IsValidIndex(TrackIndex))
		{
			return &AudioTracks[TrackIndex];
		}

	case EMediaTrackType::Caption:
		if (CaptionTracks.IsValidIndex(TrackIndex))
		{
			return &CaptionTracks[TrackIndex];
		}

	case EMediaTrackType::Video:
		if (VideoTracks.IsValidIndex(TrackIndex))
		{
			return &VideoTracks[TrackIndex];
		}

	default:
		break; // unsupported track type
	}

	return nullptr;
}


const FMfMediaTracks::FFormat* FMfMediaTracks::GetVideoFormat(int32 TrackIndex, int32 FormatIndex) const
{
	if (VideoTracks.IsValidIndex(TrackIndex))
	{
		const FTrack& Track = VideoTracks[TrackIndex];

		if (Track.Formats.IsValidIndex(FormatIndex))
		{
			return &Track.Formats[FormatIndex];
		}
	}

	return nullptr;
}


bool FMfMediaTracks::RequestSample(DWORD StreamIndex)
{
	if (SelectedAudioTrack != INDEX_NONE && AudioTracks[SelectedAudioTrack].StreamIndex == StreamIndex)
	{
		if (Samples->NumAudioSamples() >= FMediaPlayerQueueDepths::MaxAudioSinkDepth)
		{
			return false;
		}
	}
	else if (SelectedCaptionTrack != INDEX_NONE && CaptionTracks[SelectedCaptionTrack].StreamIndex == StreamIndex)
	{
		if (Samples->NumCaptionSamples() >= FMediaPlayerQueueDepths::MaxCaptionSinkDepth)
		{
			return false;
		}
	}
	else if (SelectedVideoTrack != INDEX_NONE && VideoTracks[SelectedVideoTrack].StreamIndex == StreamIndex)
	{
		if (Samples->NumVideoSamples() >= FMediaPlayerQueueDepths::MaxVideoSinkDepth)
		{
			return false;
		}
	}

#if MFMEDIATRACKS_USE_ASYNCREADER
	const HRESULT Result = SourceReader->ReadSample(StreamIndex, 0, nullptr, nullptr, nullptr, nullptr);

#else
	TComPtr<IMFSample> Sample;
	DWORD StreamFlags;
	LONGLONG Timestamp;

	const HRESULT Result = SourceReader->ReadSample(StreamIndex, 0, nullptr, &StreamFlags, &Timestamp, &Sample);
#endif

	if (FAILED(Result))
	{
		UE_LOGF(LogMfMedia, VeryVerbose, "Tracks %p: Failed to request sample for stream %i: %ls", this, StreamIndex, *MfMedia::ResultToString(Result));
		return false;
	}

#if MFMEDIATRACKS_USE_ASYNCREADER
	return true;

#else
	if (SUCCEEDED(Result))
	{
		ProcessSample(Sample, Result, StreamFlags, StreamIndex, FTimespan(Timestamp));
	}

	return false;
#endif
}


void FMfMediaTracks::UpdateAudio()
{
	if (AudioSamplePending)
	{
		return; // sample request pending
	}

	if (AudioDone || AudioSampleRange.IsEmpty() || (SelectedAudioTrack == INDEX_NONE))
	{
		return; // nothing to play
	}

	if ((LastAudioSampleTime != FTimespan::MinValue()) && !AudioSampleRange.Contains(LastAudioSampleTime))
	{
		return; // no new sample needed
	}

	#if MFMEDIATRACKS_TRACE_SAMPLES
		UE_LOGF(LogMfMedia, VeryVerbose, "Tracks %p: Requesting audio sample", this);
	#endif

	AudioSamplePending = RequestSample(AudioTracks[SelectedAudioTrack].StreamIndex);
}


void FMfMediaTracks::UpdateCaptions()
{
	if (CaptionSamplePending)
	{
		return; // sample request pending
	}

	if (CaptionDone || CaptionSampleRange.IsEmpty() || (SelectedCaptionTrack == INDEX_NONE))
	{
		return; // nothing to play
	}

	if ((LastCaptionSampleTime != FTimespan::MinValue()) && !CaptionSampleRange.Contains(LastCaptionSampleTime))
	{
		return; // no new sample needed
	}

	#if MFMEDIATRACKS_TRACE_SAMPLES
		UE_LOGF(LogMfMedia, VeryVerbose, "Tracks %p: Requesting caption sample", this);
	#endif

	CaptionSamplePending = RequestSample(CaptionTracks[SelectedCaptionTrack].StreamIndex);
}


void FMfMediaTracks::UpdateVideo()
{
	if (VideoSamplePending)
	{
		return; // sample request pending
	}

	if (VideoDone || VideoSampleRange.IsEmpty() || (SelectedVideoTrack == INDEX_NONE))
	{
		return; // nothing to play
	}

	if ((LastVideoSampleTime != FTimespan::MinValue()) && !VideoSampleRange.Contains(LastVideoSampleTime))
	{
		return; // no new sample needed
	}

	#if MFMEDIATRACKS_TRACE_SAMPLES
		UE_LOGF(LogMfMedia, VeryVerbose, "Tracks %p: Requesting video sample", this);
	#endif

	VideoSamplePending = RequestSample(VideoTracks[SelectedVideoTrack].StreamIndex);
}


#undef LOCTEXT_NAMESPACE

#if PLATFORM_MICROSOFT
	#include "Microsoft/HideMicrosoftPlatformTypes.h"
#endif

#endif //MFMEDIA_SUPPORTED_PLATFORM
