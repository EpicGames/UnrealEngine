// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MfMediaPrivate.h"

#if MFMEDIA_SUPPORTED_PLATFORM

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "HAL/CriticalSection.h"
#include "IMediaTracks.h"
#include "Internationalization/Text.h"
#include "Math/IntPoint.h"
#include "Math/Range.h"
#include "Microsoft/COMPointer.h"

#if PLATFORM_MICROSOFT
	#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#endif

#if !defined(MFMEDIATRACKS_HAS_PLATFORM_INITIALIZE)
	#define MFMEDIATRACKS_HAS_PLATFORM_INITIALIZE 0
#endif

class FMediaSamples;
class FMfMediaAudioSamplePool;
class FMfMediaTextureSamplePool; 

enum class EMediaTextureSampleFormat;

struct FMfMediaSourceReaderSample;
struct FMediaPlayerOptions;


/**
 * Track collection for Media Foundation based media players.
 */
class FMfMediaTracks
	: public IMediaTracks
{
	struct FFormat
	{
		TComPtr<IMFMediaType> InputType;
		TComPtr<IMFMediaType> OutputType;
		FString TypeName;

		struct
		{
			uint32 BitsPerSample;
			uint32 NumChannels;
			uint32 SampleRate;
		}
		Audio;

		struct 
		{
			uint32 BitRate;
			FIntPoint BufferDim;
			uint32 BufferStride;
			GUID FormatType;
			float FrameRate;
			TRange<float> FrameRates;
			FIntPoint OutputDim;
			EMediaTextureSampleFormat SampleFormat;
		}
		Video;
	};

	struct FTrack
	{
		FText DisplayName;
		TArray<FFormat> Formats;
		FString Language;
		FString Name;
		bool Protected;
		int32 SelectedFormat;
		int32 StreamIndex;
	};

public:

	/** Default constructor. */
	FMfMediaTracks();

	/** Virtual destructor. */
	virtual ~FMfMediaTracks();

public:

	/**
	 * Append track statistics information to the given string.
	 *
	 * @param OutStats The string to append the statistics to.
	 */
	void AppendStats(FString &OutStats) const;

	/**
	 * Clear the streams flags.
	 *
	 * @see GetFlags
	 */
	void ClearFlags();

	/**
	 * Get the total duration of the current media source.
	 *
	 * @return Media duration.
	 * @see GetInfo, Initialize
	 */
	FTimespan GetDuration() const;

	/**
	 * Get the current flags.
	 *
	 * @param OutMediaSourceChanged Will indicate whether the media source changed.
	 * @param OutSelectionChanged Will indicate whether the track selection changed.
	 * @see ClearFlags
	 */
	void GetFlags(bool& OutMediaSourceChanged, bool& OutSelectionChanged) const;

	/**
	 * Get the information string for the currently loaded media source.
	 *
	 * @return Info string.
	 * @see GetDuration, GetSamples
	 */
	const FString& GetInfo() const
	{
		return Info;
	}

	/**
	 * Get the current media source object.
	 *
	 * @return The media source, or nullptr if not initialized.
	 * @see GetSourceReader
	 */
	IMFMediaSource* GetMediaSource();

	/**
	 * Get the current source reader object.
	 *
	 * @return The source reader, or nullptr if not initialized.
	 * @see GetMediaSource
	 */
	IMFSourceReader* GetSourceReader();

	/**
	 * Initialize the track collection.
	 *
	 * @param InMediaSource The media source object.
	 * @param InSourceReaderCallback The player's source reader callback.
	 * @param InSamples The sample collection that receives output samples.
	 * @see IsInitialized, Shutdown
	 */
	void Initialize(IMFMediaSource* InMediaSource, IMFSourceReaderCallback* InSourceReaderCallback, const TSharedRef<FMediaSamples, ESPMode::ThreadSafe>& InSamples, const FMediaPlayerOptions* PlayerOptions);

	/**
	 * Whether this object has been initialized.
	 *
	 * @return true if initialized, false otherwise.
	 * @see Initialize, Shutdown
	 */
	bool IsInitialized() const
	{
		return (MediaSource != NULL);
	}

	/**
	 * Process a media sample from the source reader callback.
	 *
	 * @param ReaderSample The sample to process.
	 */
	void ProcessSample(IMFSample* Sample, HRESULT Status, DWORD StreamFlags, DWORD StreamIndex, FTimespan Time);

	/** Restart stream sampling. */
	void Restart();

	/**
	 * Shut down the track collection.
	 *
	 * @see Initialize, IsInitialized
	 */
	void Shutdown();

	/**
	 * Tick audio sample processing.
	 *
	 * @param Rate The current playback rate.
	 * @param Time The current presentation time.
	 * @see TickVideo
	 */
	void TickAudio(float Rate, FTimespan Time);

	/**
	 * Tick caption & video sample processing.
	 *
	 * @param Rate The current playback rate.
	 * @param Time The current presentation time.
	 * @see TickAudio
	 */
	void TickInput(float Rate, FTimespan Time);

public:

	//~ IMediaTracks interface

	virtual bool GetAudioTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaAudioTrackFormat& OutFormat) const override;
	virtual int32 GetNumTracks(EMediaTrackType TrackType) const override;
	virtual int32 GetNumTrackFormats(EMediaTrackType TrackType, int32 TrackIndex) const override;
	virtual int32 GetSelectedTrack(EMediaTrackType TrackType) const override;
	virtual FText GetTrackDisplayName(EMediaTrackType TrackType, int32 TrackIndex) const override;
	virtual int32 GetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex) const override;
	virtual FString GetTrackLanguage(EMediaTrackType TrackType, int32 TrackIndex) const override;
	virtual FString GetTrackName(EMediaTrackType TrackType, int32 TrackIndex) const override;
	virtual bool GetVideoTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaVideoTrackFormat& OutFormat) const override;
	virtual bool SelectTrack(EMediaTrackType TrackType, int32 TrackIndex) override;
	virtual bool SetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex, int32 FormatIndex) override;

protected:

	/**
	 * Adds the specified stream to the track collection.
	 *
	 * @param StreamIndex The index number of the stream in the presentation descriptor.
	 * @param OutInfo String to append debug information to.
	 * @return true on success, false otherwise.
	 */
	bool AddStreamToTracks(uint32 StreamIndex, FString& OutInfo);

	/**
	 * Get the specified audio format.
	 *
	 * @param TrackIndex Index of the audio track that contains the format.
	 * @param FormatIndex Index of the format to return.
	 * @return Pointer to format, or nullptr if not found.
	 * @see GetVideoFormat
	 */
	const FFormat* GetAudioFormat(int32 TrackIndex, int32 FormatIndex) const;

	/**
	 * Get the specified track information.
	 *
	 * @param TrackType The type of track.
	 * @param TrackIndex Index of the track to return.
	 * @return Pointer to track, or nullptr if not found.
	 */
	const FTrack* GetTrack(EMediaTrackType TrackType, int32 TrackIndex) const;

	/**
	 * Get the specified video format.
	 *
	 * @param TrackIndex Index of the video track that contains the format.
	 * @param FormatIndex Index of the format to return.
	 * @return Pointer to format, or nullptr if not found.
	 * @see GetAudioFormat
	 */
	const FFormat* GetVideoFormat(int32 TrackIndex, int32 FormatIndex) const;

	/**
	 * Request a new media sample for the specified stream.
	 *
	 * @param StreamIndex The index of the stream to request a sample for.
	 * @return true if a sample was requested, false otherwise.
	 */
	bool RequestSample(DWORD StreamIndex);

	/**
	 * Request more audio samples, if needed.
	 *
	 * @see UpdateCaptions, UpdateVideo
	 */
	void UpdateAudio();

	/**
	 * Request more caption samples, if needed.
	 *
	 * @see UpdateAudio, UpdateVideo
	 */
	void UpdateCaptions();

	/**
	 * Request more video samples, if needed.
	 *
	 * @see UpdateAudio, UpdateCaptions
	 */
	void UpdateVideo();

	/** 
	 * Platform-specific initialization, if requested
	 */
#if MFMEDIATRACKS_HAS_PLATFORM_INITIALIZE
	void PlatformInitialize();
#endif
	
private:

	/** Whether the audio track has reached the end. */
	bool AudioDone;

	/** Whether the next audio sample has been requested. */
	bool AudioSamplePending;

	/** Audio sample object pool. */
	FMfMediaAudioSamplePool* AudioSamplePool;

	/** Time range of audio samples to be requested. */
	TRange<FTimespan> AudioSampleRange;

	/** The available audio tracks. */
	TArray<FTrack> AudioTracks;

	/** Whether the caption track has reached the end. */
	bool CaptionDone;

	/** Whether the next caption sample has been requested. */
	bool CaptionSamplePending;

	/** Time range of caption samples to be requested. */
	TRange<FTimespan> CaptionSampleRange;

	/** The available caption tracks. */
	TArray<FTrack> CaptionTracks;

	/** Synchronizes write access to track arrays, selections & sinks. */
	mutable FCriticalSection CriticalSection;

	/** Media information string. */
	FString Info;

	/** Timestamp of the last processed audio sample. */
	FTimespan LastAudioSampleTime;

	/** Timestamp of the last processed caption sample. */
	FTimespan LastCaptionSampleTime;

	/** Timestamp of the last processed video sample. */
	FTimespan LastVideoSampleTime;

	/** The currently opened media. */
	TComPtr<IMFMediaSource> MediaSource;

	/** Whether the media source has changed. */
	bool MediaSourceChanged;

	/** Media sample collection that receives the output. */
	TSharedPtr<FMediaSamples, ESPMode::ThreadSafe> Samples;

	/** Index of the selected audio track. */
	int32 SelectedAudioTrack;

	/** Index of the selected caption track. */
	int32 SelectedCaptionTrack;

	/** Index of the selected video track. */
	int32 SelectedVideoTrack;

	/** Whether the track selection changed. */
	bool SelectionChanged;

	/** The source reader to use. */
	TComPtr<IMFSourceReader> SourceReader;

	/** Whether the video track has reached the end. */
	bool VideoDone;

	/** Whether the next video sample has been requested. */
	bool VideoSamplePending;

	/** Video sample object pool. */
	FMfMediaTextureSamplePool* VideoSamplePool;

	/** Time range of video samples to be requested. */
	TRange<FTimespan> VideoSampleRange;

	/** The available video tracks. */
	TArray<FTrack> VideoTracks;
};


#if PLATFORM_MICROSOFT
	#include "Microsoft/HideMicrosoftPlatformTypes.h"
#endif

#endif //MFMEDIA_SUPPORTED_PLATFORM
