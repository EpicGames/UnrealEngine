// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Modules/ModuleInterface.h"

class IMediaEventSink;
class IMediaPlayer;


/**
 * Interface for the MfMedia module.
 */
class IMfMediaModule
	: public IModuleInterface
{
public:

	/**
	 * Create an Xbox One Media Foundation based media player.
	 *
	 * @param EventSink The object that receives media events from the player.
	 * @return A new media player, or nullptr if a player couldn't be created.
	 */
	virtual TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CreatePlayer(IMediaEventSink& EventSink) = 0;

public:

	/** Virtual destructor. */
	virtual ~IMfMediaModule() = default;
};
