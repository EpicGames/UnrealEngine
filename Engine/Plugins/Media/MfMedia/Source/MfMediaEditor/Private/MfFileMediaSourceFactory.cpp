// Copyright Epic Games, Inc. All Rights Reserved.

#include "MfFileMediaSourceFactory.h"

#include "FileMediaSource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MfFileMediaSourceFactory)


/* UMfFileMediaSourceFactory structors
 *****************************************************************************/

UMfFileMediaSourceFactory::UMfFileMediaSourceFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Formats.Add(TEXT("mp4;MPEG-4 Movie"));

	SupportedClass = UFileMediaSource::StaticClass();
	bEditorImport = true;
}


/* UFactory overrides
 *****************************************************************************/

bool UMfFileMediaSourceFactory::FactoryCanImport(const FString& Filename)
{
	return true;
}


UObject* UMfFileMediaSourceFactory::FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled)
{
	UFileMediaSource* MediaSource = NewObject<UFileMediaSource>(InParent, InClass, InName, Flags);
	MediaSource->SetFilePath(CurrentFilename);

	return MediaSource;
}

