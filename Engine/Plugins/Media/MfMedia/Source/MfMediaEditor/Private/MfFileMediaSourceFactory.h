// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"

#include "MfFileMediaSourceFactory.generated.h"


/**
 * Implements a factory for UFileMediaSource objects.
 */
UCLASS(hidecategories=Object)
class UMfFileMediaSourceFactory
	: public UFactory
{
	GENERATED_UCLASS_BODY()

public:

	//~ UFactory Interface

	virtual bool FactoryCanImport(const FString& Filename) override;
	virtual UObject* FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled) override;
};
