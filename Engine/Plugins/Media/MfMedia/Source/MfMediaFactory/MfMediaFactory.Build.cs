// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MfMediaFactory : ModuleRules
	{
		public MfMediaFactory(ReadOnlyTargetRules Target) : base(Target)
		{
			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"Media",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"MediaAssets",
				});

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"Media",
					"MfMedia",
				});

			if (DoAllowHTTPSPlayback())
			{
				PrivateDefinitions.Add("MFMEDIAFACTORY_ALLOW_HTTPS=1");
			}
			else
			{
				PrivateDefinitions.Add("MFMEDIAFACTORY_ALLOW_HTTPS=0");
			}
		}

		protected virtual bool DoAllowHTTPSPlayback()
		{
			return Target.Platform.IsInGroup(UnrealPlatformGroup.Windows);
		}
	}
}
