// Copyright Epic Games, Inc. All Rights Reserved.

#include "MfMediaFactoryPrivate.h"

#include "IMediaModule.h"
#include "IMediaOptions.h"
#include "IMediaPlayerFactory.h"
#include "Internationalization/Internationalization.h"
#include "Misc/Paths.h"
#include "Misc/Guid.h"
#include "Modules/ModuleManager.h"

#include "IMfMediaModule.h"


DEFINE_LOG_CATEGORY(LogMfMediaFactory);

#define LOCTEXT_NAMESPACE "FMfMediaFactoryModule"

#define MFMEDIAFACTORY_USE_WINDOWS 0 // set this to one to enable MfMedia on Windows (experimental)


/**
 * Implements the MfMediaFactory module.
 */
class FMfMediaFactoryModule
	: public IMediaPlayerFactory
	, public IModuleInterface
{
public:

	/** Default constructor. */
	FMfMediaFactoryModule()
	{ }

public:

	//~ IMediaPlayerInfo interface

	virtual bool CanPlayUrl(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const override
	{
		return GetPlayabilityConfidenceScore(Url, Options, OutWarnings, OutErrors) > 0 ? true : false;
	}

	virtual int32 GetPlayabilityConfidenceScore(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const override
	{
		FString Scheme;
		FString Location;

		// check scheme
		if (!Url.Split(TEXT("://"), &Scheme, &Location, ESearchCase::CaseSensitive))
		{
			if (OutErrors != nullptr)
			{
				OutErrors->Add(LOCTEXT("NoSchemeFound", "No URI scheme found"));
			}

			return 0;
		}

		if (!SupportedUriSchemes.Contains(Scheme))
		{
			if (OutErrors != nullptr)
			{
				OutErrors->Add(FText::Format(LOCTEXT("SchemeNotSupported", "The URI scheme '{0}' is not supported"), FText::FromString(Scheme)));
			}

			return 0;
		}

		// check file extension
		if (Scheme == TEXT("file"))
		{
			const FString Extension = FPaths::GetExtension(Location, false);

			if (!SupportedFileExtensions.Contains(Extension))
			{
				if (OutErrors != nullptr)
				{
					OutErrors->Add(FText::Format(LOCTEXT("ExtensionNotSupported", "The file extension '{0}' is not supported"), FText::FromString(Extension)));
				}

				return 0;
			}
		}

		// check options
		if ((OutWarnings != nullptr) && (Options != nullptr))
		{
			if (Options->GetMediaOption("PrecacheFile", false) && (Scheme != TEXT("file")))
			{
				OutWarnings->Add(LOCTEXT("PrecachingNotSupported", "Precaching is supported for local files only"));
			}
		}

		return 80;
	}

	virtual TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CreatePlayer(IMediaEventSink& EventSink) override
	{
		auto MfMediaModule = FModuleManager::LoadModulePtr<IMfMediaModule>("MfMedia");
		return (MfMediaModule != nullptr) ? MfMediaModule->CreatePlayer(EventSink) : nullptr;
	}

	virtual FText GetDisplayName() const override
	{
		return LOCTEXT("MediaPlayerDisplayName", "Microsoft Media Foundation");
	}

	virtual FName GetPlayerName() const override
	{
		static FName PlayerName(TEXT("MfMedia"));
		return PlayerName;
	}

	virtual FGuid GetPlayerPluginGUID() const override
	{
		static FGuid PlayerPluginGUID(0x6a5bd063, 0xe0854163, 0x867e5978, 0xf3eaa9f2);
		return PlayerPluginGUID;
	}

	virtual const TArray<FString>& GetSupportedPlatforms() const override
	{
		return SupportedPlatforms;
	}

	virtual bool SupportsFeature(EMediaFeature Feature) const override
	{
		return ((Feature == EMediaFeature::AudioSamples) ||
				(Feature == EMediaFeature::AudioTracks) ||
				(Feature == EMediaFeature::CaptionTracks) ||
				(Feature == EMediaFeature::OverlaySamples) ||
				(Feature == EMediaFeature::VideoSamples) ||
				(Feature == EMediaFeature::VideoTracks));
	}

public:

	//~ IModuleInterface interface

	virtual void StartupModule() override
	{
		// supported file extensions
		SupportedFileExtensions.Add(TEXT("mp4"));
#if MFMEDIAFACTORY_WINDOWS && MFMEDIAFACTORY_USE_WINDOWS
		SupportedFileExtensions.Add(TEXT("3g2"));
		SupportedFileExtensions.Add(TEXT("3gp"));
		SupportedFileExtensions.Add(TEXT("3gp2"));
		SupportedFileExtensions.Add(TEXT("3gpp"));
		SupportedFileExtensions.Add(TEXT("ac3"));
		SupportedFileExtensions.Add(TEXT("aif"));
		SupportedFileExtensions.Add(TEXT("aifc"));
		SupportedFileExtensions.Add(TEXT("aiff"));
		SupportedFileExtensions.Add(TEXT("amr"));
		SupportedFileExtensions.Add(TEXT("au"));
		SupportedFileExtensions.Add(TEXT("bwf"));
		SupportedFileExtensions.Add(TEXT("caf"));
		SupportedFileExtensions.Add(TEXT("cdda"));
		SupportedFileExtensions.Add(TEXT("m4a"));
		SupportedFileExtensions.Add(TEXT("m4v"));
		SupportedFileExtensions.Add(TEXT("mov"));
		SupportedFileExtensions.Add(TEXT("qt"));
		SupportedFileExtensions.Add(TEXT("sdv"));
		SupportedFileExtensions.Add(TEXT("snd"));
		SupportedFileExtensions.Add(TEXT("wav"));
		SupportedFileExtensions.Add(TEXT("wave"));
#endif

		// supported platforms
		AddSupportedPlatform(FGuid(0x21f5cd78, 0xc2824344, 0xa0f32e55, 0x28059b27));
		AddSupportedPlatform(FGuid(0x0df604e1, 0x12e44452, 0x80bee1c7, 0x4eb934b1));
#if MFMEDIAFACTORY_WINDOWS && MFMEDIAFACTORY_USE_WINDOWS
		AddSupportedPlatform(FGuid(0xd1d5f296, 0xff834a87, 0xb20faaa9, 0xd6b8e9a6));
#endif
		AddSupportedPlatform(FGuid(0x5636fbc1, 0xd2b54f62, 0xac8e7d4f, 0xb184b45a));
		AddSupportedPlatform(FGuid(0x941259d5, 0x0a2746aa, 0xadc0ba84, 0x4790ad8a));
		AddSupportedPlatform(FGuid(0xccf05903, 0x822b47e1, 0xb2236a28, 0xdfd78817));
		AddSupportedPlatform(FGuid(0xb596ce6f, 0xd8324a9c, 0x84e9f880, 0x21322535));

		// supported schemes
		SupportedUriSchemes.Add(TEXT("file"));
#if MFMEDIAFACTORY_ALLOW_HTTPS
		SupportedUriSchemes.Add(TEXT("https"));
#endif
#if MFMEDIAFACTORY_WINDOWS && MFMEDIAFACTORY_USE_WINDOWS
		SupportedUriSchemes.Add(TEXT("http"));
		SupportedUriSchemes.Add(TEXT("httpd"));
		SupportedUriSchemes.Add(TEXT("mms"));
		SupportedUriSchemes.Add(TEXT("rtsp"));
		SupportedUriSchemes.Add(TEXT("rtspt"));
		SupportedUriSchemes.Add(TEXT("rtspu"));
#endif

		// register player factory
		auto MediaModule = FModuleManager::LoadModulePtr<IMediaModule>("Media");

		if (MediaModule != nullptr)
		{
			MediaModule->RegisterPlayerFactory(*this);
		}
	}

	virtual void ShutdownModule() override
	{
		// unregister player factory
		auto MediaModule = FModuleManager::GetModulePtr<IMediaModule>("Media");

		if (MediaModule != nullptr)
		{
			MediaModule->UnregisterPlayerFactory(*this);
		}
	}

private:
	void AddSupportedPlatform(const FGuid& PlatformGuid)
	{
		auto MediaModule = FModuleManager::GetModulePtr<IMediaModule>("Media");
		if (MediaModule)
		{
			FName PlatformName = MediaModule->GetPlatformName(PlatformGuid);
			if (!PlatformName.IsNone())
			{
				SupportedPlatforms.Add(PlatformName.ToString());
			}
		}
	}

	/** List of supported media file types. */
	TArray<FString> SupportedFileExtensions;

	/** List of platforms that the media player support. */
	TArray<FString> SupportedPlatforms;

	/** List of supported URI schemes. */
	TArray<FString> SupportedUriSchemes;
};


#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FMfMediaFactoryModule, MfMediaFactory);
